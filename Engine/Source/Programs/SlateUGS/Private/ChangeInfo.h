// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EventMonitor.h"
#include "PerforceMonitor.h"

struct FChangeInfo
{
	FDateTime Time;
	bool bHeaderRow = false;

	UGSCore::EReviewVerdict ReviewStatus = UGSCore::EReviewVerdict::Unknown;
	int Changelist = 0;
	bool bCurrentlySynced = false;
	bool bSyncingPrecompiled = false;
	bool bHasZippedBinaries = false;
	UGSCore::FChangeType ChangeType;
	FText Author;
	FString Description;

	// Todo: add Horde badges
};
