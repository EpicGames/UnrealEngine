// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateUGSApp.h"
#include "UnixCommonStartup.h"

int main(int argc, char *argv[])
{
	return CommonUnixMain(argc, argv, &RunSlateUGS);
}
