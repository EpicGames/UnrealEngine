// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScheduledSyncTimer.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "UGSLog.h"

ScheduledSyncTimer::~ScheduledSyncTimer()
{
	bFinished = true;

	if (WorkerThread != nullptr)
	{
		WorkerThread->WaitForCompletion();
		WorkerThread = nullptr;
	}
}

void ScheduledSyncTimer::Start(const FDateTime& InScheduledTime, const TFunction<void()>& InTimerElapsedCallback)
{
	check(WorkerThread == nullptr);

	bFinished            = false;
	ScheduledTime        = InScheduledTime;
	TimerElapsedCallback = InTimerElapsedCallback;

	WorkerThread = FRunnableThread::Create(this, TEXT("ScheduledSyncTimer"));
}

void ScheduledSyncTimer::Stop()
{
	bFinished = true;

	if (WorkerThread)
	{
		WorkerThread->WaitForCompletion();
		WorkerThread = nullptr;
	}
}

uint32 ScheduledSyncTimer::Run()
{
	while (!bFinished)
	{
		FDateTime Now = FDateTime::Now();

		if (IsEngineExitRequested())
		{
			bFinished = true;
		}

		if (Now > ScheduledTime)
		{
			if (TimerElapsedCallback)
			{
				TimerElapsedCallback();
			}

			// Increment our next sync 1 day from now
			ScheduledTime += FTimespan(1, 0, 0, 0);

			UE_LOGF(LogSlateUGS, Log, "Schedule: Started ScheduleTimer for %ls (%ls remaining)",
				*ScheduledTime.ToString(TEXT("%Y/%m/%d at %h:%M%a")),
				*(ScheduledTime - Now).ToString(TEXT("%h hours and %m minutes")));
		}

		// lets check every 100ms to avoid spinning this thread to hard, while still be responsive when requesting to exit
		// the ideal option would be to create a wait event tied to the bFinished bool, and sleep for longer chunks *unless* that bool
		// is flipped, but this is just simple for now
		FPlatformProcess::Sleep(0.1f);
	}

	return 0;
}
