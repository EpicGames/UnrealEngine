// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "Templates/Function.h"
#include "Misc/DateTime.h"


class ScheduledSyncTimer : FRunnable
{
public:
	~ScheduledSyncTimer();

	void Start(const FDateTime& InScheduledTime, const TFunction<void()>& InTimerElapsedCallback);
	void Stop();

private:
	virtual uint32 Run() override;

	bool bFinished = false;
	
	FDateTime ScheduledTime;
	FRunnableThread* WorkerThread = nullptr;
	TFunction<void()> TimerElapsedCallback;
};
