// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateUGSApp.h"
#include "UGSLog.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Stats/StatsSystem.h"

#include "Widgets/SGameSyncTab.h"
#include "Widgets/SWorkspaceWindow.h"
#include "Widgets/SEmptyTab.h"

#include "UGSTabManager.h"

IMPLEMENT_APPLICATION(SlateUGS, "SlateUGS");
DEFINE_LOG_CATEGORY(LogSlateUGS);

#define LOCTEXT_NAMESPACE "SlateUGS"

int RunSlateUGS(const TCHAR* CommandLine)
{
	FTaskTagScope TaskTagScope(ETaskTag::EGameThread);

	// start up the main loop
	GEngineLoop.PreInit(CommandLine);

	// Make sure all UObject classes are registered and default properties have been initialized
	ProcessNewlyLoadedUObjects();

	// Tell the module manager it may now process newly-loaded UObjects when new C++ modules are loaded
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	// crank up a normal Slate application using the platform's standalone renderer
	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());

	FSlateApplication::InitHighDPI(true);

	// set the application name
	FGlobalTabmanager::Get()->SetApplicationTitle(LOCTEXT("AppTitle", "Unreal Game Sync"));

	FAppStyle::SetAppStyleSetName(FAppStyle::GetAppStyleSetName());

	// new scope to allow TabManager to go out of scope before the Engine is dead
	{
		// Build the slate UI for the program window
		UGSTabManager TabManager;
		TabManager.ConstructTabs();

		// loop while the server does the rest
		while (!IsEngineExitRequested())
		{
			BeginExitIfRequested();

			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			UE::Stats::FStats::AdvanceFrame(false);
			FTSTicker::GetCoreTicker().Tick(FApp::GetDeltaTime());
			FSlateApplication::Get().PumpMessages();
			FSlateApplication::Get().Tick();
			TabManager.Tick();
			FPlatformProcess::Sleep(0.01f);

			GFrameCounter++;
		}
	}

	FCoreDelegates::OnExit.Broadcast();
	FSlateApplication::Shutdown();
	FModuleManager::Get().UnloadModulesAtShutdown();

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();

	return 0;
}

#undef LOCTEXT_NAMESPACE
