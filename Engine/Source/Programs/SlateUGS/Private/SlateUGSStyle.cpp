// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateUGSStyle.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/SlateStyleRegistry.h"

FName FSlateUGSStyle::StyleName("SlateUGSStyle");
TUniquePtr<FSlateUGSStyle> FSlateUGSStyle::Inst(nullptr);

const FName& FSlateUGSStyle::GetStyleSetName() const
{
	return StyleName;
}

const FSlateUGSStyle& FSlateUGSStyle::Get()
{
	if (!Inst.IsValid())
	{
		Inst = TUniquePtr<FSlateUGSStyle>(new FSlateUGSStyle);
	}
	return *(Inst.Get());
}

void FSlateUGSStyle::Shutdown()
{
	Inst.Reset();
}

FSlateUGSStyle::FSlateUGSStyle() : FSlateStyleSet(StyleName)
{
	SetParentStyleName(FAppStyle::GetAppStyleSetName());
	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("HordeBadge.Color.Unknown", FLinearColor(0.50f, 0.50f, 0.50f));
	Set("HordeBadge.Color.Error",   FLinearColor(1.00f, 0.10f, 0.10f));
	Set("HordeBadge.Color.Warning", FLinearColor(1.00f, 0.75f, 0.00f));
	Set("HordeBadge.Color.Success", FLinearColor(0.25f, 1.00f, 0.25f));
	Set("HordeBadge.Color.Pending", FLinearColor(0.25f, 0.75f, 1.00f));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FSlateUGSStyle::~FSlateUGSStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
