// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

class FSlateUGSStyle : public FSlateStyleSet
{
public:
	virtual const FName& GetStyleSetName() const override;

	static const FSlateUGSStyle& Get();
	static void Shutdown();

	~FSlateUGSStyle();

private:
	FSlateUGSStyle();

	static FName StyleName;
	static TUniquePtr<FSlateUGSStyle> Inst;
};
