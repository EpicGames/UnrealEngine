// Copyright Epic Games, Inc. All Rights Reserved.

#include "UGSTab.h"

#include "UGSLog.h"
#include "ChangeInfo.h"
#include "CreateClientTask.h"
#include "FindStreamsTask.h"
#include "UGSTabManager.h"

#include "Utility.h"
#include "BuildStep.h"
#include "DetectProjectSettingsTask.h"
#include "FindStreamsTask.h"
#include "PerforceMonitor.h"
#include "EventMonitor.h"

#include "Widgets/SModalTaskWindow.h"
#include "Widgets/SPopupTextWindow.h"
#include "Widgets/SLogWidget.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Application/SlateApplication.h"

#include "Internationalization/Regex.h"

#define LOCTEXT_NAMESPACE "UGSTab"

UGSTab::UGSTab() : TabArgs(nullptr, FTabId()),
				   TabWidget(SNew(SDockTab)),
				   EmptyTabView(SNew(SEmptyTab, this)),
				   GameSyncTabView(SNew(SGameSyncTab, this)),
				   bHasQueuedMessages(false),
				   bNeedUpdateGameTabBuildList(false)
{
	Initialize(nullptr);
}

UGSTab::~UGSTab()
{
	if (Workspace)
	{
		Workspace->CancelUpdate();
	}
}

void UGSTab::Initialize(TSharedPtr<UGSCore::FUserSettings> InUserSettings)
{
	UserSettings = InUserSettings;

	TabWidget->SetContent(EmptyTabView);
	TabWidget->SetLabel(FText(LOCTEXT("TabName", "Select a Project")));

	PerforceClient = MakeShared<UGSCore::FPerforceConnection>(TEXT(""), TEXT(""), TEXT(""));
}

void UGSTab::SetTabManager(UGSTabManager* InTabManager)
{
	TabManager = InTabManager;
}

const TSharedRef<SDockTab> UGSTab::GetTabWidget()
{
	return TabWidget;
}

void UGSTab::SetTabArgs(FSpawnTabArgs InTabArgs)
{
	TabArgs = InTabArgs;
}

FSpawnTabArgs UGSTab::GetTabArgs() const
{
	return TabArgs;
}

namespace
{
	// TODO super hacky... fix up later
	#if PLATFORM_WINDOWS
		const TCHAR* HostPlatform = TEXT("Win64");
	#elif PLATFORM_MAC
		const TCHAR* HostPlatform = TEXT("Mac");
	#else
		const TCHAR* HostPlatform = TEXT("Linux");
	#endif

	class FLineWriter : public UGSCore::FLineBasedTextWriter
	{
	public:
		virtual void FlushLine(const FString& Line) override
		{
			UE_LOGF(LogSlateUGS, Log, "%ls", *Line);
		}
	};

	FString GetEditorExePath(UGSCore::EBuildConfig Config, TSharedPtr<UGSCore::FDetectProjectSettingsTask> DetectSettings)
	{
	#if PLATFORM_WINDOWS
		FString ExeFileName = TEXT("UnrealEditor.exe");
	#else
		FString ExeFileName = TEXT("UnrealEditor");
	#endif

		if (Config != UGSCore::EBuildConfig::DebugGame && Config != UGSCore::EBuildConfig::Development)
		{
	#if PLATFORM_WINDOWS
			ExeFileName = FString::Printf(TEXT("UnrealEditor-%s-%s.exe"), HostPlatform, *UGSCore::ToString(Config));
	#else
			ExeFileName = FString::Printf(TEXT("UnrealEditor-%s-%s"), HostPlatform, *UGSCore::ToString(Config));
	#endif
		}
		return DetectSettings->BranchDirectoryName / "Engine" / "Binaries" / HostPlatform / ExeFileName;
	}

	static FString FormatUserName(FString UserName)
	{
		TStringBuilder<256> NormalUserName;
		for (int Index = 0; Index < UserName.Len(); Index++)
		{
			if (Index == 0 || UserName[Index - 1] == '.')
			{
				NormalUserName.AppendChar(FChar::ToUpper(UserName[Index]));
			}
			else if (UserName[Index] == '.')
			{
				NormalUserName.AppendChar(' ');
			}
			else
			{
				NormalUserName.AppendChar(FChar::ToLower(UserName[Index]));
			}
		}

		return NormalUserName.ToString();
	}
}

bool UGSTab::IsProjectLoaded() const
{
	return !ProjectFileName.IsEmpty();
}

bool UGSTab::OnWorkspaceChosen(const FString& Project)
{
	bool bIsDataValid = FPaths::FileExists(Project); // Todo: Check that the project file is also associated with a workspace
	if (bIsDataValid)
	{
		ProjectFileName = Project;

		if (SetupWorkspace())
		{
			GameSyncTabView->SetStreamPathText(FText::FromString(DetectSettings->StreamName));
			GameSyncTabView->SetChangelistText(WorkspaceSettings->CurrentChangeNumber);
			GameSyncTabView->SetProjectPathText(FText::FromString(ProjectFileName));
			TabWidget->SetContent(GameSyncTabView);
			TabWidget->SetLabel(FText::FromString(DetectSettings->StreamName));

			return true;
		}
	}

	return false;
}

void UGSTab::RefreshBuildList()
{
	Log->Logf(TEXT("Refreshing build list..."));
	PerforceMonitor->Refresh();
}

void UGSTab::CancelSync()
{
	if (Workspace->IsBusy())
	{
		Log->Logf(TEXT("Canceling sync/build..."));
		Workspace->CancelUpdate();
	}
}

TMap<FString, FString> UGSTab::GetWorkspaceVariables() const
{
	UGSCore::EBuildConfig EditorBuildConfig = GetEditorBuildConfig();

	TMap<FString, FString> Variables;
	Variables.Add("BranchDir", DetectSettings->BranchDirectoryName);
	Variables.Add("ProjectDir", FPaths::GetPath(DetectSettings->NewSelectedFileName));
	Variables.Add("ProjectFile", DetectSettings->NewSelectedFileName);

	// Todo: These might not be called "UE4*" anymore
	Variables.Add("UE4EditorExe", GetEditorExePath(EditorBuildConfig, DetectSettings));
	Variables.Add("UE4EditorCmdExe", GetEditorExePath(EditorBuildConfig, DetectSettings).Replace(TEXT(".exe"), TEXT("-Cmd.exe")));
	Variables.Add("UE4EditorConfig", UGSCore::ToString(EditorBuildConfig));
	Variables.Add("UE4EditorDebugArg", (EditorBuildConfig == UGSCore::EBuildConfig::Debug || EditorBuildConfig == UGSCore::EBuildConfig::DebugGame)? " -debug" : "");

	return Variables;
}

UGSCore::EBuildConfig UGSTab::GetEditorBuildConfig() const
{
	return ShouldSyncPrecompiledEditor() ? UGSCore::EBuildConfig::Development : UserSettings->CompiledEditorBuildConfig;
}

bool UGSTab::ShouldSyncPrecompiledEditor() const
{
	return UserSettings->bSyncPrecompiledEditor && PerforceMonitor->HasZippedBinaries();
}

TArray<FString> UGSTab::GetAllStreamNames() const
{
	TArray<FString> Result;

	const TSharedRef<UGSCore::FModalTaskResult> TaskResult = ExecuteModalTask(
		FSlateApplication::Get().GetActiveModalWindow(),
		MakeShared<UGSCore::FFindStreamsTask>(PerforceClient.ToSharedRef(), MakeShared<FLineWriter>(), Result, TEXT("//*/*")),
		LOCTEXT("FindingStreamsTitle", "Finding Streams"),
		LOCTEXT("FindingStreamsCaption", "Finding streams, please wait..."));
	
	if (TaskResult->Failed())
	{
		FSlateApplication::Get().AddModalWindow(
			SNew(SPopupTextWindow)
				.TitleText(LOCTEXT("FindStreamsErrorTitle", "Error"))
				.BodyText(TaskResult->GetMessage()),
			FSlateApplication::Get().GetActiveModalWindow());
		return TArray<FString>();
	}

	return Result;
}

// Honestly ... seems ... super hacky/hardcoded. With out all these you assert when trying to merge build targets sooo a bit odd
// TODO Need to do each of these ... per ... platform??
TMap<FGuid, UGSCore::FCustomConfigObject> UGSTab::GetDefaultBuildStepObjects(const FString& EditorTargetName)
{
	TArray<UGSCore::FBuildStep> DefaultBuildSteps;
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0x01F66060, 0x73FA4CC8, 0x9CB3E217, 0xFBBA954E), 0, TEXT("Compile UnrealHeaderTool"), TEXT("Compiling UnrealHeaderTool..."), 1, TEXT("UnrealHeaderTool"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	FString ActualEditorTargetName = (EditorTargetName.Len() > 0) ? EditorTargetName : "UnrealEditor";
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0xF097FF61, 0xC9164058, 0x839135B4, 0x6C3173D5), 1, FString::Printf(TEXT("Compile %s"), *ActualEditorTargetName), FString::Printf(TEXT("Compiling %s..."), *ActualEditorTargetName), 10, ActualEditorTargetName, HostPlatform, UGSCore::ToString(UserSettings->CompiledEditorBuildConfig), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0xC6E633A1, 0x956F4AD3, 0xBC956D06, 0xD131E7B4), 2, TEXT("Compile ShaderCompileWorker"), TEXT("Compiling ShaderCompileWorker..."), 1, TEXT("ShaderCompileWorker"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0x24FFD88C, 0x79014899, 0x9696AE10, 0x66B4B6E8), 3, TEXT("Compile UnrealLightmass"), TEXT("Compiling UnrealLightmass..."), 1, TEXT("UnrealLightmass"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0xFFF20379, 0x06BF4205, 0x8A3EC534, 0x27736688), 4, TEXT("Compile CrashReportClient"), TEXT("Compiling CrashReportClient..."), 1, TEXT("CrashReportClient"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0x89FE8A79, 0xD2594C7B, 0xBFB468F7, 0x218B91C2), 5, TEXT("Compile UnrealInsights"), TEXT("Compiling UnrealInsights..."), 1, TEXT("UnrealInsights"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0x46312669, 0x5069428D, 0x8D72C241, 0x6C5A322E), 6, TEXT("Launch UnrealInsights"), TEXT("Running UnrealInsights..."), 1, TEXT("UnrealInsights"), HostPlatform, TEXT("Shipping"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0xBB48CA5B, 0x56824432, 0x824DC451, 0x336A6523), 7, TEXT("Compile Zen Dashboard"), TEXT("Compile ZenDashboard Step..."), 1, TEXT("ZenDashboard"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0x586CC0D3, 0x39144DF9, 0xACB62C02, 0xCD9D4FC6), 8, TEXT("Launch Zen Dashboard"), TEXT("Running Zen Dashboard..."), 1, TEXT("ZenDashboard"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0x91C2A429, 0xC39149B4, 0x92A54E6B, 0xE71E0F00), 9, TEXT("Compile SwitchboardListener"), TEXT("Compiling SwitchboardListener..."), 1, TEXT("SwitchboardListener"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0x5036C75B, 0x8DF04329, 0x82A1869D, 0xD2D48605), 10, TEXT("Compile UnrealMultiUserServer"), TEXT("Compiling UnrealMultiUserServer..."), 1, TEXT("UnrealMultiUserServer"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0x274B89C3, 0x9DC64465, 0xA50840AB, 0xC4593CC2), 11, TEXT("Compile UnrealMultiUserSlateServer"), TEXT("Compiling UnrealMultiUserSlateServer..."), 1, TEXT("UnrealMultiUserSlateServer"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));
	DefaultBuildSteps.Add(UGSCore::FBuildStep(FGuid(0xF7F0C4C7, 0x55514485, 0xA961F3E4, 0xFD9D138C), 12, TEXT("Compile InterchangeWorker"), TEXT("Compiling InterchangeWorker..."), 1, TEXT("InterchangeWorker"), HostPlatform, TEXT("Development"), TEXT(""), !ShouldSyncPrecompiledEditor()));

	TMap<FGuid, UGSCore::FCustomConfigObject> DefaultBuildStepObjects;
	for (const UGSCore::FBuildStep& DefaultBuildStep : DefaultBuildSteps)
	{
		DefaultBuildStepObjects.Add(DefaultBuildStep.UniqueId, DefaultBuildStep.ToConfigObject());
	}

	return DefaultBuildStepObjects;
}

void UGSTab::OnSyncChangelist(int Changelist)
{
	// Options on what to do with workspace when updating it
	UGSCore::EWorkspaceUpdateOptions Options = UGSCore::EWorkspaceUpdateOptions::Sync | UGSCore::EWorkspaceUpdateOptions::GenerateProjectFiles;
	if (UserSettings->bAutoResolveConflicts)
	{
		Options |= UGSCore::EWorkspaceUpdateOptions::AutoResolveChanges;
	}
	if (UserSettings->bUseIncrementalBuilds)
	{
		Options |= UGSCore::EWorkspaceUpdateOptions::UseIncrementalBuilds;
	}
	if (UserSettings->bBuildAfterSync)
	{
		Options |= UGSCore::EWorkspaceUpdateOptions::Build;
	}
	if (UserSettings->bBuildAfterSync && UserSettings->bRunAfterSync)
	{
		Options |= UGSCore::EWorkspaceUpdateOptions::RunAfterSync;
	}
	if (UserSettings->bOpenSolutionAfterSync)
	{
		Options |= UGSCore::EWorkspaceUpdateOptions::OpenSolutionAfterSync;
	}

	TSharedRef<UGSCore::FWorkspaceUpdateContext, ESPMode::ThreadSafe> Context = MakeShared<UGSCore::FWorkspaceUpdateContext, ESPMode::ThreadSafe>(
		Changelist,
		Options,
		CombinedSyncFilter,
		GetDefaultBuildStepObjects(DetectSettings->NewProjectEditorTarget),
		ProjectSettings->BuildSteps,
		TSet<FGuid>(),
		GetWorkspaceVariables());

	if (UserSettings->bSyncPrecompiledEditor)
	{
		FString ArchivePath;
		if (PerforceMonitor->TryGetArchivePathForChangeNumber(Changelist, ArchivePath))
		{
			Context->Options |= UGSCore::EWorkspaceUpdateOptions::SyncArchives;
			Context->ArchiveTypeToDepotPath.Add(TPair<FString, FString>("Editor", ArchivePath));
		}
	}

	// Update the workspace with the Context!
	Workspace->Update(Context);
}

void UGSTab::OnBuildWorkspace()
{
	UGSCore::EWorkspaceUpdateOptions Options = UGSCore::EWorkspaceUpdateOptions::Build;

	TSharedRef<UGSCore::FWorkspaceUpdateContext, ESPMode::ThreadSafe> Context = MakeShared<UGSCore::FWorkspaceUpdateContext, ESPMode::ThreadSafe>(
		-1,
		Options,
		CombinedSyncFilter,
		GetDefaultBuildStepObjects(DetectSettings->NewProjectEditorTarget),
		ProjectSettings->BuildSteps,
		TSet<FGuid>(),
		GetWorkspaceVariables());

	// Update the workspace with the Context!
	Workspace->Update(Context);
}

void UGSTab::OnViewInSwarmClicked(int Changelist) const
{
	FString SwarmURL = FString::Printf(TEXT("https://p4-swarm.epicgames.net/changes/%i"), Changelist);

	Log->Logf(TEXT("Opening Swarm to %s"), *SwarmURL);
	FPlatformProcess::LaunchURL(*SwarmURL, nullptr, nullptr);
}

void UGSTab::OnCopyChangeListClicked(int Changelist) const
{
	Log->Logf(TEXT("Copying CL %i to clipboard"), Changelist);
	FPlatformApplicationMisc::ClipboardCopy(*FString::FromInt(Changelist));
}

void UGSTab::OnMoreInfoClicked(int Changelist) const
{
	Log->Logf(TEXT("Opening CL %i in p4v"), Changelist);
	PerforceClient->OpenP4VC(FString::Printf(TEXT("change %i"), Changelist));
}

void UGSTab::OnOpenPerforceClicked() const
{
	Log->Logf(TEXT("Opening %s in p4v"), *ProjectFileName);
	PerforceClient->OpenP4V();
}

void UGSTab::OnSyncLatest()
{
	int ChangeNumber = -1;
	FEvent* AbortEvent = FPlatformProcess::GetSynchEventFromPool(true);

	if (PerforceClient->LatestChangeList(ChangeNumber, AbortEvent, *Log.Get()))
	{
		OnSyncChangelist(ChangeNumber);
	}

	FPlatformProcess::ReturnSynchEventToPool(AbortEvent);
}

void UGSTab::OnSyncFilterWindowSaved(
	const TArray<FString>& SyncViewCurrent,
	const TArray<FGuid>& SyncIncludedCategoriesCurrent,
	const TArray<FGuid>& SyncExcludedCategoriesCurrent,
	const TArray<FString>& SyncViewAll,
	const TArray<FGuid>& SyncIncludedCategoriesAll,
	const TArray<FGuid>& SyncExcludedCategoriesAll)
{
	WorkspaceSettings->SyncView = SyncViewCurrent;
	WorkspaceSettings->SyncIncludedCategories = SyncIncludedCategoriesCurrent;
	WorkspaceSettings->SyncExcludedCategories = SyncExcludedCategoriesCurrent;

	UserSettings->SyncView = SyncViewAll;
	UserSettings->SyncIncludedCategories = SyncIncludedCategoriesAll;
	UserSettings->SyncExcludedCategories = SyncExcludedCategoriesAll;

	UserSettings->Save();
}

void UGSTab::OnOpenExplorer()
{
	Log->Logf(TEXT("Opening %s in the explorer"), *FPaths::GetPath(ProjectFileName));
	FPlatformProcess::ExploreFolder(*FPaths::GetPath(ProjectFileName));
}

void UGSTab::OnOpenEditor()
{
	UGSCore::EBuildConfig EditorBuildConfig = GetEditorBuildConfig();

	FString EditorPath = GetEditorExePath(EditorBuildConfig, DetectSettings);

	Log->Logf(TEXT("Running Editor %s %s"), *EditorPath, *ProjectFileName);
	EditorProcessHandle = FPlatformProcess::CreateProc(*EditorPath, *ProjectFileName, true, false, false, nullptr, 0, nullptr, nullptr, nullptr);
}

void UGSTab::OnCreateWorkspace(const FString& WorkspaceName, const FString& Stream, const FString& RootDirectory) const
{
	const TSharedRef<FLineWriter> CreateWorkspaceLog = MakeShared<FLineWriter>();
	FEvent* AbortEvent = FPlatformProcess::GetSynchEventFromPool(true);

	TSharedPtr<UGSCore::FPerforceInfoRecord> PerforceInfo;
	PerforceClient->Info(PerforceInfo, AbortEvent, *CreateWorkspaceLog);

	TMap<FString, FString> Tags;
	Tags.Add(TEXT("client"), WorkspaceName);
	Tags.Add(TEXT("Owner"), PerforceInfo->UserName);
	Tags.Add(TEXT("Host"), PerforceInfo->HostName);
	Tags.Add(TEXT("Root"), RootDirectory);
	const UGSCore::FPerforceClientRecord ClientRecord(Tags);

	const TSharedRef<UGSCore::FModalTaskResult> TaskResult = ExecuteModalTask(
		FSlateApplication::Get().GetActiveModalWindow(),
		MakeShared<UGSCore::FCreateClientTask>(PerforceClient.ToSharedRef(), MakeShared<FLineWriter>(), ClientRecord, Stream),
		LOCTEXT("CreatingProjectTitle", "Creating Project"),
		LOCTEXT("CreatingProjectCaption", "Creating project, please wait..."));
}

void UGSTab::QueueMessageForMainThread(TFunction<void()> Function)
{
	FScopeLock Lock(&CriticalSection);

	MessageQueue.Add(Function);
	bHasQueuedMessages = true;
}

void UGSTab::Tick()
{
	if (bHasQueuedMessages)
	{
		bHasQueuedMessages = false;

		FScopeLock Lock(&CriticalSection);
		for (const TFunction<void()>& MessageFunction : MessageQueue)
		{
			MessageFunction();
		}

		MessageQueue.Empty();
	}

	if (bNeedUpdateGameTabBuildList)
	{
		bNeedUpdateGameTabBuildList = false;

		UpdateGameTabBuildList();
	}

	// if we have spawned an editor process, check if its closed to reap it
	if (EditorProcessHandle.IsValid())
	{
		if (!FPlatformProcess::IsProcRunning(EditorProcessHandle))
		{
			FPlatformProcess::CloseProc(EditorProcessHandle);
			EditorProcessHandle.Reset();
		}
	}
}

namespace
{
	bool ShouldShowChange(const UGSCore::FPerforceChangeSummary& Change, const TArray<FString>& ExcludeChanges)
	{
		for (const FString& ExcludeChange : ExcludeChanges)
		{
			FRegexPattern RegexPattern(*ExcludeChange, ERegexPatternFlags::CaseInsensitive);
			FRegexMatcher RegexMatcher(RegexPattern, Change.Description);

			if (RegexMatcher.FindNext())
			{
				return false;
			}
		}

		if (Change.User == TEXT("buildmachine") && Change.Description.Contains(TEXT("lightmaps")))
		{
			return false;
		}

		return true;
	}
}

bool UGSTab::ShouldIncludeInReviewedList(const TSet<int>& PromotedChangeNumbers, int ChangeNumber) const
{
	if (PromotedChangeNumbers.Contains(ChangeNumber))
	{
		return true;
	}

	TSharedPtr<UGSCore::FEventSummary> Review;
	if (EventMonitor->TryGetSummaryForChange(ChangeNumber, Review))
	{
		if (Review->LastStarReview.IsValid() && Review->LastStarReview->Type == UGSCore::EEventType::Starred)
		{
			return true;
		}

		if (Review->Verdict == UGSCore::EReviewVerdict::Good || Review->Verdict == UGSCore::EReviewVerdict::Mixed)
		{
			return true;
		}
	}

	return false;
}

void UGSTab::UpdateGameTabBuildList()
{
	TArray<TSharedRef<UGSCore::FPerforceChangeSummary, ESPMode::ThreadSafe>> Changes = PerforceMonitor->GetChanges();
	TArray<TSharedPtr<FChangeInfo>> ChangeInfos;

	// do we need to store these for something?
	TSet<int> PromotedChangeNumbers = PerforceMonitor->GetPromotedChangeNumbers();

	TArray<FString> ExcludeChanges;
	TSharedPtr<UGSCore::FCustomConfigFile, ESPMode::ThreadSafe> ProjectConfigFile = Workspace->GetProjectConfigFile();
	if (ProjectConfigFile.IsValid())
	{
		ProjectConfigFile->TryGetValues(TEXT("Options.ExcludeChanges"), ExcludeChanges);
	}

	bool bFirstChange = true;
	bool bOnlyShowReviewed = false;
	//  bool bOnlyShowReviewed = OnlyShowReviewedCheckBox.Checked;

	for (int ChangeIdx = 0; ChangeIdx < Changes.Num(); ChangeIdx++)
	{
		const UGSCore::FPerforceChangeSummary& Change = Changes[ChangeIdx].Get();
		if (ShouldShowChange(Change, ExcludeChanges) || PromotedChangeNumbers.Contains(Change.Number))
		{
			if (!bOnlyShowReviewed || (!EventMonitor->IsUnderInvestigation(Change.Number) && (ShouldIncludeInReviewedList(PromotedChangeNumbers, Change.Number) || bFirstChange)))
			{
				bFirstChange = false;

				FDateTime DisplayTime = Change.Date;
				if (UserSettings->bShowLocalTimes)
				{
					DisplayTime = (DisplayTime - DetectSettings->ServerTimeZone).GetDate();
				}

				TSharedPtr<UGSCore::FEventSummary> Review;
				UGSCore::EReviewVerdict Status = UGSCore::EReviewVerdict::Unknown;
				if (EventMonitor->TryGetSummaryForChange(Change.Number, Review) && Review->LastStarReview.IsValid())
				{
					if (Review->LastStarReview->Type == UGSCore::EEventType::Starred)
					{
						Status = UGSCore::EReviewVerdict::Good;
					}
					else
					{
						Status = Review->Verdict;
					}
				}

				bool bHasZippedBinaries = false;
				FString Temp;
				if (PerforceMonitor->TryGetArchivePathForChangeNumber(Change.Number, Temp))
				{
					bHasZippedBinaries = true;
				}

				UGSCore::FChangeType ChangeType;
				PerforceMonitor->TryGetChangeType(Change.Number, ChangeType);

				if (ChangeIdx == 0 ||
					Changes[ChangeIdx - 1]->Date.GetDayOfYear() != Changes[ChangeIdx]->Date.GetDayOfYear())
				{
					TSharedPtr<FChangeInfo> HeaderRow = MakeShareable(new FChangeInfo);

					HeaderRow->Time = DisplayTime;
					HeaderRow->bHeaderRow = true;

					ChangeInfos.Add(HeaderRow);
				}

				bool bCurrentSyncedChange = Change.Number == WorkspaceSettings->CurrentChangeNumber;

				ChangeInfos.Add(MakeShareable(new FChangeInfo
				{
					DisplayTime,
					false,
					Status,
					Change.Number,
					bCurrentSyncedChange,
					ShouldSyncPrecompiledEditor(),
					bHasZippedBinaries,
					ChangeType,
					FText::FromString(FormatUserName(Change.User)),
					Change.Description
				}));
			}
		}
	}

	GameSyncTabView->AddHordeBuilds(ChangeInfos);
}

bool UGSTab::IsSyncing() const
{
	return Workspace->IsBusy();
}

FString UGSTab::GetSyncProgress() const
{
	TTuple<FString, float> CurrentProgress = Workspace->GetCurrentProgress();

	if (CurrentProgress.Value > 0.0f)
	{
		return FString::Printf(TEXT("%s %.2f%%"), *CurrentProgress.Key, CurrentProgress.Value * 100.0f);
	}

	return CurrentProgress.Key;
}

TArray<UGSCore::FWorkspaceSyncCategory> UGSTab::GetSyncCategories(SyncCategoryType CategoryType) const
{
	TArray<UGSCore::FWorkspaceSyncCategory> SyncCategories;
	TMap<FGuid, UGSCore::FWorkspaceSyncCategory> Categories = Workspace->GetSyncCategories();
	TArray<FGuid> SyncIncludedCategories;
	TArray<FGuid> SyncExcludedCategories;

	if (CategoryType == SyncCategoryType::CurrentWorkspace)
	{
		SyncIncludedCategories = WorkspaceSettings->SyncIncludedCategories;
		SyncExcludedCategories = WorkspaceSettings->SyncExcludedCategories;
	}
	else
	{
		SyncIncludedCategories = UserSettings->SyncIncludedCategories;
		SyncExcludedCategories = UserSettings->SyncExcludedCategories;
	}

	for (const FGuid& Guid : SyncIncludedCategories)
	{
		if (UGSCore::FWorkspaceSyncCategory* Category = Categories.Find(Guid))
		{
			Category->bEnable = true;
		}
	}

	for (const FGuid& Guid : SyncExcludedCategories)
	{
		if (UGSCore::FWorkspaceSyncCategory* Category = Categories.Find(Guid))
		{
			Category->bEnable = false;
		}
	}

	Categories.GenerateValueArray(SyncCategories);

	return SyncCategories;
}

TArray<FString> UGSTab::GetSyncViews(SyncCategoryType CategoryType) const
{
	if (CategoryType == SyncCategoryType::CurrentWorkspace)
	{
		return WorkspaceSettings->SyncView;
	}

	return UserSettings->SyncView;
}

const TArray<FString>& UGSTab::GetCombinedSyncFilter() const
{
	return CombinedSyncFilter;
}

UGSTabManager* UGSTab::GetTabManager() const
{
	return TabManager;
}

TSharedPtr<UGSCore::FUserSettings> UGSTab::GetUserSettings() const
{
	return UserSettings;
}

// This is getting called on a thread, lock our stuff up
void UGSTab::OnWorkspaceSyncComplete(TSharedRef<UGSCore::FWorkspaceUpdateContext, ESPMode::ThreadSafe> WorkspaceContext, UGSCore::EWorkspaceUpdateResult SyncResult, const FString& StatusMessage)
{
	FScopeLock Lock(&CriticalSection);

	if (SyncResult == UGSCore::EWorkspaceUpdateResult::Success)
	{
		WorkspaceSettings->CurrentChangeNumber   = Workspace->GetCurrentChangeNumber();
		WorkspaceSettings->LastBuiltChangeNumber = Workspace->GetLastBuiltChangeNumber();

		// Queue up setting the changelist text on the main thread
		QueueMessageForMainThread([this] {
			GameSyncTabView->SetChangelistText(WorkspaceSettings->CurrentChangeNumber);
		});

		// TODO hacky, but allows us to highlight which CL we have synced to in the list
		bNeedUpdateGameTabBuildList = true;
	}

	WorkspaceSettings->LastSyncResult        = SyncResult;
	WorkspaceSettings->LastSyncResultMessage = StatusMessage;
	WorkspaceSettings->LastSyncTime          = WorkspaceContext->StartTime;

	// TODO check this is valid, may be off
	WorkspaceSettings->LastSyncDurationSeconds = (FDateTime::UtcNow() - WorkspaceContext->StartTime).GetSeconds();

	UserSettings->Save();
}

bool UGSTab::SetupWorkspace()
{
	ProjectFileName = UGSCore::FUtility::GetPathWithCorrectCase(ProjectFileName);

	// TODO likely should also log this on an Empty tab... so we can show logging info when we are loading things
	DetectSettings = MakeShared<UGSCore::FDetectProjectSettingsTask>(PerforceClient.ToSharedRef(), ProjectFileName, MakeShared<FLineWriter>());

	TSharedRef<UGSCore::FModalTaskResult> Result = ExecuteModalTask(
		FSlateApplication::Get().GetActiveModalWindow(),
		DetectSettings.ToSharedRef(),
		LOCTEXT("OpeningProjectTitle", "Opening Project"),
		LOCTEXT("OpeningProjectCaption", "Opening project, please wait..."));

	if (Result->Failed())
	{
		FSlateApplication::Get().AddModalWindow(
			SNew(SPopupTextWindow)
				.TitleText(LOCTEXT("SetupWorkspaceErrorTitle", "Error"))
				.BodyText(Result->GetMessage()),
			FSlateApplication::Get().GetActiveModalWindow());
		return false;
	}

	PerforceClient    = DetectSettings->PerforceClient;
	WorkspaceSettings = UserSettings->FindOrAddWorkspace(*DetectSettings->BranchClientPath);
	ProjectSettings   = UserSettings->FindOrAddProject(*DetectSettings->NewSelectedClientFileName);

	// Check if the project we've got open in this workspace is the one we're actually synced to
	int CurrentChangeNumber = -1;
	if (WorkspaceSettings->CurrentProjectIdentifier == DetectSettings->NewSelectedProjectIdentifier)
	{
		CurrentChangeNumber = WorkspaceSettings->CurrentChangeNumber;
	}

	FString ClientKey = DetectSettings->BranchClientPath.Replace(*FString::Printf(TEXT("//%s/"), *PerforceClient->ClientName), TEXT(""));
	if (ClientKey.EndsWith(TEXT("/")))
	{
		ClientKey = ClientKey.Left(ClientKey.Len() - 1);
	}

	FString DataFolder = FString(FPlatformProcess::UserSettingsDir()) / TEXT("UnrealGameSync");

	FString ProjectLogBaseName         = DataFolder / FString::Printf(TEXT("%s@%s"), *PerforceClient->ClientName, *ClientKey.Replace(TEXT("/"), TEXT("$")));
	FString TelemetryProjectIdentifier = UGSCore::FPerforceUtils::GetClientOrDepotDirectoryName(*DetectSettings->NewSelectedProjectIdentifier);
	FString SelectedProjectIdentifier  = DetectSettings->NewSelectedProjectIdentifier;

	FString LogFileName = DataFolder / FPaths::GetPath(ProjectFileName) + TEXT(".sync.log");
	GameSyncTabView->SetSyncLogLocation(LogFileName); // Todo: if SetSyncLogLocation fails, then it failed to create a log file and may need to handle that
	GameSyncTabView->GetSyncLog()->AppendLine(TEXT("Creating log at: ") + LogFileName);
	Log = MakeShared<FLogWidgetTextWriter>(GameSyncTabView->GetSyncLog().ToSharedRef());

	Workspace = MakeShared<UGSCore::FWorkspace>(
		PerforceClient.ToSharedRef(),
		DetectSettings->BranchDirectoryName,
		ProjectFileName,
		DetectSettings->BranchClientPath,
		DetectSettings->NewSelectedClientFileName,
		SelectedProjectIdentifier,
		CurrentChangeNumber,
		WorkspaceSettings->LastBuiltChangeNumber,
		TelemetryProjectIdentifier,
		Log.ToSharedRef());

	Workspace->OnUpdateComplete = [this] (TSharedRef<UGSCore::FWorkspaceUpdateContext, ESPMode::ThreadSafe> WorkspaceContext, UGSCore::EWorkspaceUpdateResult SyncResult, const FString& StatusMessage) {
		OnWorkspaceSyncComplete(WorkspaceContext, SyncResult, StatusMessage);
	};

	CombinedSyncFilter = UGSCore::FUserSettings::GetCombinedSyncFilter(
		Workspace->GetSyncCategories(),
		UserSettings->SyncView,
		UserSettings->SyncExcludedCategories,
		WorkspaceSettings->SyncView,
		WorkspaceSettings->SyncExcludedCategories);

	// Setup our Perforce and Event monitoring threads
	FString BranchClientPath = DetectSettings->BranchDirectoryName;
	FString SelectedClientFileName = DetectSettings->NewSelectedClientFileName;

	// TODO create callback functions that will be queued for the main thread to generate and update the main table view
	PerforceMonitor = MakeShared<UGSCore::FPerforceMonitor>(PerforceClient.ToSharedRef(), BranchClientPath, SelectedClientFileName, SelectedProjectIdentifier, ProjectLogBaseName + ".p4.log");
	PerforceMonitor->OnUpdate = [this]{ QueueMessageForMainThread([this] { UpdateGameTabBuildList(); }); };

	//PerforceMonitor->OnUpdateMetadata = [this]{ QueueMessageForMainThread([this] { UpdateGameTabBuildList(); }); }; //MessageQueue.Add([this]{ UpdateBuildMetadata(); }); };
	PerforceMonitor->OnChangeTypeQueryFinished = [this]{ QueueMessageForMainThread([this] { UpdateGameTabBuildList(); }); };
	PerforceMonitor->OnStreamChange = [this]{ printf("PerforceMonitor->OnStreamChange\n"); }; // MessageQueue.Add([this]{ Owner->StreamChanged(this); }); };

	/* TODO figure out if this is even working, and if so how to correctly use this
	 */
	FString SqlConnectionString;
	EventMonitor = MakeShared<UGSCore::FEventMonitor>(SqlConnectionString, UGSCore::FPerforceUtils::GetClientOrDepotDirectoryName(*SelectedProjectIdentifier), PerforceClient->UserName, ProjectLogBaseName + ".review.log");
	EventMonitor->OnUpdatesReady = [this]{ printf("EventMonitor->OnUpdatesReady\n"); }; //MessageQueue.Add([this]{ UpdateReviews(); }); };

	// Start the threads
	PerforceMonitor->Start();
	EventMonitor->Start();

	return true;
}

#undef LOCTEXT_NAMESPACE
