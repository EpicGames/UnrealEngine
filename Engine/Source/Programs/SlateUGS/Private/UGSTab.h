// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FindStreamsTask.h"
#include "Widgets/SEmptyTab.h"
#include "Widgets/SGameSyncTab.h"
#include "Widgets/Docking/SDockTab.h"

#include "Workspace.h"
#include "UserSettings.h"

namespace UGSCore
{
	struct FUserWorkspaceSettings;
	struct FUserProjectSettings;
	class FDetectProjectSettingsTask;
	class FPerforceMonitor;
	class FEventMonitor;
}

class UGSTabManager;
class FLogWidgetTextWriter;

enum SyncCategoryType
{
	CurrentWorkspace,
	AllWorkspaces
};

class UGSTab
{
public:
	UGSTab();
	~UGSTab();

	void Initialize(TSharedPtr<UGSCore::FUserSettings> UserSettings);

	const TSharedRef<SDockTab> GetTabWidget();
	FSpawnTabArgs GetTabArgs() const;

	void SetTabManager(UGSTabManager* InTabManager);
	void SetTabArgs(FSpawnTabArgs InTabArgs);

	void Tick();

	bool IsProjectLoaded() const;

	// Slate callbacks
	bool OnWorkspaceChosen(const FString& Project);
	void OnSyncChangelist(int Changelist);
	void OnViewInSwarmClicked(int Changelist) const;
	void OnCopyChangeListClicked(int Changelist) const;
	void OnMoreInfoClicked(int Changelist) const;
	void OnOpenPerforceClicked() const;
	void OnSyncLatest();
	void OnSyncFilterWindowSaved(
		const TArray<FString>& SyncViewCurrent,
		const TArray<FGuid>& SyncIncludedCategoriesCurrent,
		const TArray<FGuid>& SyncExcludedCategoriesCurrent,
		const TArray<FString>& SyncViewAll,
		const TArray<FGuid>& SyncIncludedCategoriesAll,
		const TArray<FGuid>& SyncExcludedCategoriesAll);
	void OnBuildWorkspace();
	void OnOpenExplorer();
	void OnOpenEditor();
	void OnCreateWorkspace(const FString& WorkspaceName, const FString& Stream, const FString& RootDirectory) const;

	// Accessors
	bool IsSyncing() const;
	FString GetSyncProgress() const;
	const TArray<FString>& GetCombinedSyncFilter() const;
	TArray<UGSCore::FWorkspaceSyncCategory> GetSyncCategories(SyncCategoryType CategoryType) const;
	TArray<FString> GetSyncViews(SyncCategoryType CategoryType) const;
	UGSTabManager* GetTabManager() const;
	TSharedPtr<UGSCore::FUserSettings> GetUserSettings() const;
	bool ShouldSyncPrecompiledEditor() const;
	TArray<FString> GetAllStreamNames() const;

	void UpdateGameTabBuildList();
	void RefreshBuildList();
	void CancelSync();

private:
	void OnWorkspaceSyncComplete(
		TSharedRef<UGSCore::FWorkspaceUpdateContext, ESPMode::ThreadSafe> WorkspaceContext,
		UGSCore::EWorkspaceUpdateResult SyncResult,
		const FString& StatusMessage);

	TMap<FString, FString> GetWorkspaceVariables() const;
	UGSCore::EBuildConfig GetEditorBuildConfig() const;
	TMap<FGuid, UGSCore::FCustomConfigObject> GetDefaultBuildStepObjects(const FString& EditorTargetName);

	// Allows the queuing of functions from threads to be run on the main thread
	void QueueMessageForMainThread(TFunction<void()> Function);

	bool ShouldIncludeInReviewedList(const TSet<int>& PromotedChangeNumbers, int ChangeNumber) const;

	// Core functions
	bool SetupWorkspace();

	FCriticalSection CriticalSection;

	// Slate Data
	FSpawnTabArgs TabArgs;
	TSharedRef<SDockTab> TabWidget;
	TSharedRef<SEmptyTab> EmptyTabView;
	TSharedRef<SGameSyncTab> GameSyncTabView;

	// Core data
	UGSTabManager* TabManager = nullptr;
	FString ProjectFileName;
	TSharedPtr<UGSCore::FWorkspace> Workspace;
	TSharedPtr<UGSCore::FPerforceConnection> PerforceClient;
	TSharedPtr<UGSCore::FUserWorkspaceSettings> WorkspaceSettings;
	TSharedPtr<UGSCore::FUserProjectSettings> ProjectSettings;
	TSharedPtr<UGSCore::FDetectProjectSettingsTask> DetectSettings;
	TArray<FString> CombinedSyncFilter;
	TSharedPtr<UGSCore::FUserSettings> UserSettings;
	TSharedPtr<FLogWidgetTextWriter> Log;

	// Monitoring threads
	TSharedPtr<UGSCore::FPerforceMonitor> PerforceMonitor;
	TSharedPtr<UGSCore::FEventMonitor> EventMonitor;

	// Queue for handling callbacks from multiple threads that need to be called on the main thread
	TArray<TFunction<void()>> MessageQueue;
	std::atomic<bool> bHasQueuedMessages;

	std::atomic<bool> bNeedUpdateGameTabBuildList;

	// If we launch the editor, keep track of it to reap when its exited
	FProcHandle EditorProcessHandle;
};
