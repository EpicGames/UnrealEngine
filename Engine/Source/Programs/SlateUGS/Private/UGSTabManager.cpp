// Copyright Epic Games, Inc. All Rights Reserved.

#include "UGSTabManager.h"
#include "Containers/Array.h"
#include "UGSLog.h"
#include "HAL/FileManager.h"

#include "UserSettings.h"

UGSTabManager::UGSTabManager()
{
	FString DataFolder = FString(FPlatformProcess::UserSettingsDir()) / TEXT("UnrealGameSync");
	IFileManager::Get().MakeDirectory(*DataFolder);

	UserSettings = MakeShared<UGSCore::FUserSettings>(*(DataFolder / TEXT("UnrealGameSync_Slate.ini")));
}

void UGSTabManager::ConstructTabs()
{
	TSharedRef<FTabManager::FStack> TabStack = FTabManager::NewStack();
	TSharedPtr<FGlobalTabmanager> TabManager = FGlobalTabmanager::Get();

	for (int TabIndex = 0; TabIndex < MaxTabs; TabIndex++)
	{
		const FName TabId = GetTabId(TabIndex);
		TabManager->RegisterTabSpawner(TabId, FOnSpawnTab::CreateLambda([this, TabIndex] (const FSpawnTabArgs& Args) { return SpawnTab(TabIndex, Args); }));

		// Leave the first tab opened, close the rest
		if (TabIndex == 0)
		{
			TabStack->AddTab(TabId, ETabState::OpenedTab);
		}
		else
		{
			TabStack->AddTab(TabId, ETabState::ClosedTab);
		}
	}

	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("UGS_Layout")
	->AddArea
	(
		FTabManager::NewArea(1230, 900)
		->Split(TabStack)
	);

	TabManager->RestoreFrom(Layout, nullptr);

	SetupScheduledSync();
}

void UGSTabManager::Tick()
{
	for (UGSTab& Tab : Tabs)
	{
		Tab.Tick();
	}

	if (bScheduledTimerElapsed)
	{
		bScheduledTimerElapsed = false;

		ScheduleTimerElapsed();
	}
}

TSharedRef<SDockTab> UGSTabManager::SpawnTab(int Index, const FSpawnTabArgs& Args)
{
	Tabs[Index].Initialize(UserSettings);
	Tabs[Index].SetTabArgs(Args);
	Tabs[Index].SetTabManager(this);

	return Tabs[Index].GetTabWidget();
}

void UGSTabManager::SetupScheduledSync()
{
	StopScheduledSyncTimer();

	if (UserSettings->bScheduleEnabled)
	{
		StartScheduledSyncTimer();
	}
}

void UGSTabManager::StartScheduledSyncTimer()
{
	bScheduledTimerElapsed = false;

	// We need to grab the current date info for the ticks, as our timer will increment by days, as well as
	// we will use ::Now() to check if we our timer is ready to signal
	FDateTime CurrentTime   = FDateTime::Now();
	FDateTime ScheduledTime = CurrentTime.GetDate() + UserSettings->ScheduleTime;

	// If our CurrentTime is greater then our ScheduleTime, add 1 day to our ScheduleTime to hit the next occurrence of that time
	if (CurrentTime > ScheduledTime)
	{
		ScheduledTime += FTimespan(1, 0, 0, 0);
	}

	// Add or subtract some fudge time to distribute some of the sync time around versus all at the same time
	int FudgeMinutes = 10;
	FTimespan FudgeTime = FTimespan::FromMinutes(FMath::RandRange(FudgeMinutes * -100, FudgeMinutes * 100) / 100.0f);

	ScheduledTime += FudgeTime;

	UE_LOGF(LogSlateUGS, Log, "Schedule: Started ScheduleTimer for %ls (%ls remaining)",
		*ScheduledTime.ToString(TEXT("%Y/%m/%d at %h:%M%a")),
		*(ScheduledTime - CurrentTime).ToString(TEXT("%h hours and %m minutes")));

	SyncTimer.Start(ScheduledTime, [this] {
		// flip this atomic so on the next tick we can sync on the main thread
		bScheduledTimerElapsed = true;
	});
}

void UGSTabManager::StopScheduledSyncTimer()
{
	SyncTimer.Stop();
}

void UGSTabManager::ScheduleTimerElapsed()
{
	TSharedPtr<FGlobalTabmanager> TabManager = FGlobalTabmanager::Get();
	for (int TabIndex = 0; TabIndex < MaxTabs; TabIndex++)
	{
		FName TabId = GetTabId(TabIndex);
		if (TabManager->FindExistingLiveTab(TabId).IsValid() && Tabs[TabIndex].IsProjectLoaded())
		{
			// TODO we need a way to sync based on good vs only latest change (which could be broken)
			Tabs[TabIndex].OnSyncLatest();
		}
	}

	// flip this back for the next sync timer that will happen in ~24 hours if UGS is left open
	bScheduledTimerElapsed = false;
}

// Todo: replace this super hacky way of fetching the first available closed tab
void UGSTabManager::ActivateTab()
{
	TSharedPtr<FGlobalTabmanager> TabManager = FGlobalTabmanager::Get();
	for (int TabIndex = 0; TabIndex < MaxTabs; TabIndex++)
	{
		FName TabId = GetTabId(TabIndex);
		if (TabManager->FindExistingLiveTab(TabId).IsValid())
		{
			continue;
		}
		if (TabManager->TryInvokeTab(TabId, false).IsValid())
		{
			Tabs[TabIndex].Initialize(UserSettings);
			return;
		}
	}

	UE_LOGF(LogSlateUGS, Warning, "Cannot activate any more tabs");
}

FName UGSTabManager::GetTabId(int TabIndex) const
{
	return FName(FString(TEXT("UGS Tab: ")) + FString::FromInt(TabIndex));
}
