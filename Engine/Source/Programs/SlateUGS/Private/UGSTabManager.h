// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UGSTab.h"
#include "ScheduledSyncTimer.h"
#include "Widgets/Docking/SDockTab.h"

namespace UGSCore
{
	struct FUserSettings;
}

class UGSTabManager
{
public:
	UGSTabManager();
	void ConstructTabs();

	void Tick();

	TSharedRef<SDockTab> SpawnTab(int Index, const FSpawnTabArgs& Args);
	void ActivateTab();
	FName GetTabId(int TabIndex) const;

	// Will setup a schedule sync, as well as tear down an existing one and make a new one
	void SetupScheduledSync();
	void StopScheduledSyncTimer();

private:
	void StartScheduledSyncTimer();
	void ScheduleTimerElapsed();

	static constexpr int MaxTabs = 10;
	TStaticArray<UGSTab, MaxTabs> Tabs;

	TSharedPtr<UGSCore::FUserSettings> UserSettings;

	std::atomic<bool> bScheduledTimerElapsed;
	ScheduledSyncTimer SyncTimer;
};
