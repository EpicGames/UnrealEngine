// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEmptyTab.h"

#include "Framework/Application/SlateApplication.h"
#include "SWorkspaceWindow.h"
#include "SPrimaryButton.h"
#include "SlateUGSStyle.h"

#include "UGSTab.h"

#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "UGSEmptyTab"

void SEmptyTab::Construct(const FArguments& InArgs, UGSTab* InTab)
{
	Tab = InTab;

	this->ChildSlot
	[
		SNew(SBox)
		.WidthOverride(800)
		.HeightOverride(600)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(10.0f, 0.0f)
			[
				SNew(SImage)
				.Image(FSlateUGSStyle::Get().GetBrush("AppIcon.Small"))
				.DesiredSizeOverride(CoreStyleConstants::Icon64x64)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(10.0f, 0.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("GetStartedText", "To get started, open an Unreal project file on your hard drive."))
				]
				+SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("OpenProject", "Open Project"))
						.OnClicked(this, &SEmptyTab::OnOpenProjectClicked)
					]
				]
			]
		]
	];
}

FReply SEmptyTab::OnOpenProjectClicked()
{
	FSlateApplication& SlateApplication = FSlateApplication::Get();
	SlateApplication.AddModalWindow(SNew(SWorkspaceWindow, Tab), Tab->GetTabArgs().GetOwnerWindow(), false);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
