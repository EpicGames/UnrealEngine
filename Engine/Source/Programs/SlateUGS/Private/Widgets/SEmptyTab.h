// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UGSTab;

class SEmptyTab final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEmptyTab) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UGSTab* InTab);

private:
	FReply OnOpenProjectClicked();

	UGSTab* Tab = nullptr;
};
