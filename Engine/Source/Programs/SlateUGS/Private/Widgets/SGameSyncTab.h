// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "ChangeInfo.h"

class UGSTab;
class SLogWidget;

class SBuildDataRow final : public SMultiColumnTableRow<TSharedPtr<FChangeInfo>>
{
	SLATE_BEGIN_ARGS(SBuildDataRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FChangeInfo>& Item);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

protected:
	TSharedPtr<FChangeInfo> CurrentItem;
};

class SGameSyncTab final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGameSyncTab) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UGSTab* InTab);

	// We need access to the SyncLog when creating the Workspace
	// TODO: think of a better way to do this
	TSharedPtr<SLogWidget> GetSyncLog() const;
	bool SetSyncLogLocation(const FString& LogFileName);

	void SetStreamPathText(FText StreamPath);
	void SetChangelistText(int Changelist);
	void SetProjectPathText(FText ProjectPath);

	void AddHordeBuilds(const TArray<TSharedPtr<FChangeInfo>>& Builds);

private:
	TSharedRef<ITableRow> GenerateHordeBuildTableRow(TSharedPtr<FChangeInfo> InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	TSharedRef<SWidget> MakeSyncButtonDropdown();

	// Widget callbacks
	TSharedPtr<SWidget> OnRightClickedBuild();

	TSharedPtr<SListView<TSharedPtr<FChangeInfo>>> HordeBuildsView;
	TArray<TSharedPtr<FChangeInfo>> HordeBuilds;

	TSharedPtr<SLogWidget> SyncLog;

	TSharedPtr<STextBlock> StreamPathText;
	TSharedPtr<STextBlock> ChangelistText;
	TSharedPtr<STextBlock> ProjectPathText;
	TSharedPtr<STextBlock> SyncProgressText;

	static constexpr float HordeBuildRowHorizontalPadding = 10.0f;
	static constexpr float HordeBuildRowVerticalPadding = 2.5f;
	static constexpr float HordeBuildRowExtraIconPadding = 10.0f;

	UGSTab* Tab = nullptr;
};
