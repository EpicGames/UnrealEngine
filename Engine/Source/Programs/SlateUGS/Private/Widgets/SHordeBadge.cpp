// Copyright Epic Games, Inc. All Rights Reserved.

#include "SHordeBadge.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "SlateUGSStyle.h"

namespace
{
	FLinearColor GetBadgeColor(EBadgeState BadgeState)
	{
		switch (BadgeState)
		{
			case EBadgeState::Unknown:
			default: // Fall through
				return FSlateUGSStyle::Get().GetColor("HordeBadge.Color.Unknown");
			case EBadgeState::Error:
				return FSlateUGSStyle::Get().GetColor("HordeBadge.Color.Error");
			case EBadgeState::Warning:
				return FSlateUGSStyle::Get().GetColor("HordeBadge.Color.Warning");
			case EBadgeState::Success:
				return FSlateUGSStyle::Get().GetColor("HordeBadge.Color.Success");
			case EBadgeState::Pending:
				return FSlateUGSStyle::Get().GetColor("HordeBadge.Color.Pending");
		}
	}
}

void SHordeBadge::Construct(const FArguments& InArgs)
{
	SButton::Construct(SButton::FArguments()
		// .ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton")) // Todo: replace with FSlateUGSStyle button style
		.ButtonColorAndOpacity(GetBadgeColor(InArgs._BadgeState.Get()))
		.OnClicked(InArgs._OnClicked)
		.ForegroundColor(FSlateColor::UseStyle())
		.ContentPadding(FMargin(1.0f))
		.HAlign(HAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(InArgs._Text)
				.Visibility(InArgs._Text.IsSet() ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText"))
			]
		]
	);
}
