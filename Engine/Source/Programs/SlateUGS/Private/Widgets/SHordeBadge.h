// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SButton.h"

enum class EBadgeState
{
	Unknown,

	Error,
	Warning,
	Success,
	Pending,
};

class SHordeBadge : public SButton
{
public:
	SLATE_BEGIN_ARGS(SHordeBadge)
		: _BadgeState(EBadgeState::Unknown)
		{}

		/** The text to display in the button. */
		SLATE_ATTRIBUTE(FText, Text)

		/** The state for the badge which determines its color */
		SLATE_ATTRIBUTE(EBadgeState, BadgeState)

		/** Called when the button is clicked */
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

};
