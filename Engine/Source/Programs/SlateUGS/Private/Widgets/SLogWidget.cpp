// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLogWidget.h"
#include "Framework/Text/SlateTextRun.h"
#include "SlateOptMacros.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "UnrealGameSync"

//// FLogWidgetTextLayoutMarshaller ////

FLogWidgetTextLayoutMarshaller::FLogWidgetTextLayoutMarshaller()
	: TextLayout(nullptr)
{
}

FLogWidgetTextLayoutMarshaller::~FLogWidgetTextLayoutMarshaller()
{
}

void FLogWidgetTextLayoutMarshaller::SetText(const FString& SourceString, FTextLayout& TargetTextLayout)
{
	TextLayout = &TargetTextLayout;

	for (const TSharedRef<FString>& Line : Lines)
	{
		TextLayout->AddLine(FSlateTextLayout::FNewLineData(Line, TArray<TSharedRef<IRun>>()));
	}
}

void FLogWidgetTextLayoutMarshaller::GetText(FString& TargetString, const FTextLayout& SourceTextLayout)
{
	SourceTextLayout.GetAsText(TargetString);
}

void FLogWidgetTextLayoutMarshaller::Clear()
{
	Lines.Empty();
	MakeDirty();
}

void FLogWidgetTextLayoutMarshaller::AppendLine(const FString& Line)
{
	TSharedRef<FString> NewLine = MakeShared<FString>(Line);
	Lines.Add(NewLine);

	if (TextLayout)
	{
		// Remove the "default" line that's added for an empty text box.
		if (Lines.Num() == 1)
		{
			TextLayout->ClearLines();
		}

		FTextBlockStyle Style = FTextBlockStyle::GetDefault();
		Style.ColorAndOpacity = FSlateColor(FLinearColor::White);

		TArray<TSharedRef<IRun>> Runs;
		Runs.Add(FSlateTextRun::Create(FRunInfo(), NewLine, Style));
		TextLayout->AddLine(FSlateTextLayout::FNewLineData(NewLine, Runs));
	}
}

int32 FLogWidgetTextLayoutMarshaller::GetNumLines() const
{
	return Lines.Num();
}

//// SLogWidget ////

SLogWidget::SLogWidget()
	: bIsUserScrolled(false)
	, LogWriter(nullptr)
{
}

SLogWidget::~SLogWidget()
{
}

void SLogWidget::Construct(const FArguments& InArgs)
{
	MessagesTextMarshaller = MakeShared<FLogWidgetTextLayoutMarshaller>();

	ChildSlot
	[
		SNew(SBorder)
		[
			SAssignNew(MessagesTextBox, SMultiLineEditableTextBox)
				.Style(FAppStyle::Get(), "Log.TextBox")
				.Padding(10.0f)
				.ForegroundColor(FLinearColor::Gray)
				.Marshaller(MessagesTextMarshaller)
				.IsReadOnly(true)
				.AlwaysShowScrollbars(true)
				.OnVScrollBarUserScrolled(this, &SLogWidget::OnScroll)

		]
	];

	RegisterActiveTimer(0.03f, FWidgetActiveTimerDelegate::CreateSP(this, &SLogWidget::OnTimerElapsed));
}

bool SLogWidget::OpenFile(const TCHAR* NewLogFileName)
{
	CloseFile();
	// Clear(); // Todo: calling Clear() breaks some things (like causing the logs below to not be appended), but maybe we're doing something wrong?

	TArray<FString> InitialLines;
	if (FFileHelper::LoadFileToStringArray(InitialLines, NewLogFileName))
	{
		for (const FString& InitialLine : InitialLines)
		{
			AppendLine(InitialLine);
		}
	}

	LogWriter = IFileManager::Get().CreateFileWriter(NewLogFileName);
	if (LogWriter == nullptr)
	{
		return false;
	}

	LogFileName = NewLogFileName;
	return true;
}

void SLogWidget::CloseFile()
{
	if (LogWriter != nullptr)
	{
		delete LogWriter;
		LogWriter = nullptr;
	}
}

void SLogWidget::Clear()
{
	if (LogWriter != nullptr)
	{
		delete LogWriter;
		LogWriter = IFileManager::Get().CreateFileWriter(*LogFileName);
	}
	MessagesTextMarshaller->Clear();
}

void SLogWidget::ScrollToEnd()
{
	MessagesTextBox->ScrollTo(FTextLocation(MessagesTextMarshaller->GetNumLines() - 1));
	bIsUserScrolled = false;
}

void SLogWidget::AppendLine(const FString& Line)
{
	FScopeLock Lock(&CriticalSection);
	QueuedLines.Add(Line);
}

void SLogWidget::OnScroll(float ScrollOffset)
{
	bIsUserScrolled = ScrollOffset < 1.0 && !FMath::IsNearlyEqual(ScrollOffset, 1.0f);
}

EActiveTimerReturnType SLogWidget::OnTimerElapsed(double CurrentTime, float DeltaTime)
{
	FScopeLock Lock(&CriticalSection);
	for (const FString& QueuedLine : QueuedLines)
	{
		if (LogWriter != nullptr)
		{
			FTCHARToUTF8 UTF8String(*QueuedLine, QueuedLine.Len());
			LogWriter->Serialize((UTF8CHAR*)UTF8String.Get(), UTF8String.Length() * sizeof(UTF8CHAR));

			const ANSICHAR LineTerminator[] = LINE_TERMINATOR_ANSI;
			LogWriter->Serialize((void*)LineTerminator, sizeof(LineTerminator));
		}
		MessagesTextMarshaller->AppendLine(QueuedLine);
	}

	if (!bIsUserScrolled)
	{
		ScrollToEnd();
	}

	QueuedLines.Empty();
	return EActiveTimerReturnType::Continue;
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
