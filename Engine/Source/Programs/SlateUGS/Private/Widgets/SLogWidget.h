// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateCore.h"
#include "Framework/Text/BaseTextLayoutMarshaller.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

#include "OutputAdapters.h"

class FLogWidgetTextLayoutMarshaller : public FBaseTextLayoutMarshaller
{
public:
	FLogWidgetTextLayoutMarshaller();
	virtual ~FLogWidgetTextLayoutMarshaller();

	// ITextLayoutMarshaller
	virtual void SetText(const FString& SourceString, FTextLayout& TargetTextLayout) override;
	virtual void GetText(FString& TargetString, const FTextLayout& SourceTextLayout) override;

	void Clear();
	void AppendLine(const FString& Line);
	int32 GetNumLines() const;

private:
	TArray<TSharedRef<FString>> Lines;
	FTextLayout* TextLayout;
};

class SLogWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLogWidget) { }
	SLATE_END_ARGS()

	SLogWidget();
	~SLogWidget();

	void Construct( const FArguments& InArgs );

	bool OpenFile(const TCHAR* NewLogFileName);
	void CloseFile();

	void Clear();
	void ScrollToEnd();
	void AppendLine(const FString& Line);

protected:
	FString LogFileName;
	FCriticalSection CriticalSection;
	TArray<FString> QueuedLines;
	TSharedPtr<FLogWidgetTextLayoutMarshaller> MessagesTextMarshaller;
	TSharedPtr<SMultiLineEditableTextBox> MessagesTextBox;
	bool bIsUserScrolled;
	FArchive* LogWriter;

	void OnScroll(float ScrollOffset);
	EActiveTimerReturnType OnTimerElapsed(double CurrentTime, float DeltaTime);
};

class FLogWidgetTextWriter : public UGSCore::FLineBasedTextWriter
{
public:
	FLogWidgetTextWriter(const TSharedRef<SLogWidget>& InLogWidget)
		: LogWidget(InLogWidget)
	{
	}

	virtual void FlushLine(const FString& Line) override
	{
		LogWidget->AppendLine(Line);
	}

private:
	TSharedRef<SLogWidget> LogWidget;
};
