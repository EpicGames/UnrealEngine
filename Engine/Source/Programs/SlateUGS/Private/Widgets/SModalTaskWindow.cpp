// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModalTaskWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SThrobber.h"
#include "ModalTask.h"

SModalTaskWindow::SModalTaskWindow()
{
	AbortEvent = FPlatformProcess::GetSynchEventFromPool(true);
	CloseEvent = FPlatformProcess::GetSynchEventFromPool(true);
}

SModalTaskWindow::~SModalTaskWindow()
{
	if (Thread != nullptr)
	{
		Thread->WaitForCompletion();
		delete Thread;
	}

	FPlatformProcess::ReturnSynchEventToPool(AbortEvent);
	FPlatformProcess::ReturnSynchEventToPool(CloseEvent);
}

void SModalTaskWindow::Construct(const FArguments& InArgs)
{
	SWindow::Construct(
		SWindow::FArguments()
		.Title(InArgs._Title)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.HasCloseButton(false)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.MaxHeight(160.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.FillHeight(1.0f)
			.Padding(FMargin(100.0f, 8.0f))
			[
				SNew(STextBlock)
				.Text(InArgs._Message)
			]
			+SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Top)
			[
				SNew(SThrobber)
			]
		]
	);

	Task = InArgs._Task;

	RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SModalTaskWindow::OnTickTimer));
	Thread = FRunnableThread::Create(this, TEXT("ModalTask"));
}

EActiveTimerReturnType SModalTaskWindow::OnTickTimer(double CurrentTime, float DeltaTime)
{
	if (CloseEvent->Wait(FTimespan::Zero()))
	{
		RequestDestroyWindow();
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

uint32 SModalTaskWindow::Run()
{
	Result = Task->Run(AbortEvent);
	CloseEvent->Trigger();

	return 0;
}

TSharedRef<UGSCore::FModalTaskResult> ExecuteModalTask(TSharedPtr<SWidget> Parent, TSharedRef<UGSCore::IModalTask> Task, const FText& InTitle, const FText& InMessage)
{
	TSharedRef<SModalTaskWindow> Window =
		SNew(SModalTaskWindow)
		.Title(InTitle)
		.Message(InMessage)
		.Task(Task);

	FSlateApplication::Get().AddModalWindow(Window, Parent, false);

	return Window->Result.ToSharedRef();
}
