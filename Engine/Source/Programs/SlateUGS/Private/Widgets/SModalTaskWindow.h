// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateCore.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

namespace UGSCore
{
	class IModalTask;
	class FModalTaskResult;
}

class SModalTaskWindow : public SWindow, private FRunnable
{
public:
	SLATE_BEGIN_ARGS(SModalTaskWindow) {}
		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(FText, Message)
		SLATE_ARGUMENT(TSharedPtr<UGSCore::IModalTask>, Task)
	SLATE_END_ARGS()

	TSharedPtr<UGSCore::FModalTaskResult> Result;

	SModalTaskWindow();
	~SModalTaskWindow();

	void Construct(const FArguments& InArgs);
	EActiveTimerReturnType OnTickTimer(double CurrentTime, float DeltaTime);

private:
	FEvent* AbortEvent;
	FEvent* CloseEvent;
	FRunnableThread* Thread;
	TSharedPtr<UGSCore::IModalTask> Task;

	virtual uint32 Run() override;
};

TSharedRef<UGSCore::FModalTaskResult> ExecuteModalTask(TSharedPtr<SWidget> Parent, TSharedRef<UGSCore::IModalTask> Task, const FText& InTitle, const FText& InMessage);
