// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNewWorkspaceWindow.h"

#include "DesktopPlatformModule.h"
#include "SlateUGSStyle.h"
#include "Framework/Application/SlateApplication.h"

#include "UGSTab.h"
#include "SGameSyncTab.h"
#include "SPrimaryButton.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/SSelectStreamWindow.h"

#define LOCTEXT_NAMESPACE "UGSNewWorkspaceWindow"

void SNewWorkspaceWindow::Construct(const FArguments& InArgs, TSharedPtr<SWorkspaceWindow> InParent, UGSTab* InTab)
{
	Tab = InTab;
	Parent = InParent;

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("WindowTitleNew", "New Workspace"))
	.SizingRule(ESizingRule::FixedSize)
	.ClientSize(FVector2D(800, 200))
	[
		SNew(SBox)
		.Padding(30.0f, 15.0f, 30.0f, 0.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHeader)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CustomView", "Settings"))
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			.Padding(40.0f, 20.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(0.25f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("StreamText", "Stream:"))
					]
					+SVerticalBox::Slot()
					.Padding(0.0f, 7.5f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RootDirectoryText", "Root Directory:"))
					]
					+SVerticalBox::Slot()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NameText", "Name:"))
					]
				]
				+SHorizontalBox::Slot()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						[
							SAssignNew(StreamTextBox, SEditableTextBox)
						]
						+SHorizontalBox::Slot()
						.FillWidth(0.225f)
						.HAlign(HAlign_Right)
						[
							SNew(SButton)
							.Text(LOCTEXT("BrowseStreamButtonText", "Browse..."))
							.OnClicked(this, &SNewWorkspaceWindow::OnBrowseStreamClicked)
						]
					]
					+SVerticalBox::Slot()
					.Padding(0.0f, 7.5f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						[
							SAssignNew(RootDirTextBox, SEditableTextBox)
						]
						+SHorizontalBox::Slot()
						.FillWidth(0.225f)
						.HAlign(HAlign_Right)
						[
							SNew(SButton)
							.Text(LOCTEXT("BrowseRootDirectoryButtonText", "Browse..."))
							.OnClicked(this, &SNewWorkspaceWindow::OnBrowseRootDirectoryClicked)
						]
					]
					+SVerticalBox::Slot()
					[
						SAssignNew(WorkspaceNameTextBox, SEditableTextBox)
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				.Padding(10.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 10.0f, 0.0f)
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("CreateButtonText", "Create"))
						.OnClicked(this, &SNewWorkspaceWindow::OnCreateClicked)
						.IsEnabled(this, &SNewWorkspaceWindow::IsCreateButtonEnabled)
					]
					+SHorizontalBox::Slot()
					[
						SNew(SButton)
						.Text(LOCTEXT("CancelButtonText", "Cancel"))
						.OnClicked(this, &SNewWorkspaceWindow::OnCancelClicked)
					]
				]
			]
		]
	]);
}

FReply SNewWorkspaceWindow::OnBrowseStreamClicked()
{
	FString OutStreamPath;
	FSlateApplication::Get().AddModalWindow(SNew(SSelectStreamWindow, Tab, &OutStreamPath), SharedThis(this), false);

	StreamTextBox->SetText(FText::FromString(*OutStreamPath));

	return FReply::Handled();
}

FReply SNewWorkspaceWindow::OnBrowseRootDirectoryClicked()
{
	bool bFolderSelected = FDesktopPlatformModule::Get()->OpenDirectoryDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
		LOCTEXT("OpenDialogTitle", "Select a parent folder for your workspace").ToString(),
		RootDirPreviousFolder,
		RootDirPreviousFolder
	);
	
	if (bFolderSelected)
	{
		RootDirTextBox->SetText(FText::FromString(FPaths::ConvertRelativePathToFull(RootDirPreviousFolder)));
	}

	return FReply::Handled();
}

FReply SNewWorkspaceWindow::OnCreateClicked()
{
	Tab->OnCreateWorkspace(
		WorkspaceNameTextBox->GetText().ToString(),
		StreamTextBox->GetText().ToString(),
		RootDirTextBox->GetText().ToString()
	);

	Parent->SetWorkspaceTextBox(WorkspaceNameTextBox->GetText());

	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();

	return FReply::Handled();
}

FReply SNewWorkspaceWindow::OnCancelClicked()
{
	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();
	return FReply::Handled();
}

bool SNewWorkspaceWindow::IsCreateButtonEnabled() const
{
	const FString StreamPath = StreamTextBox->GetText().ToString();
	const bool bStreamIsValid = StreamPath.Contains(TEXT("//")) && StreamPath.RightChop(2).Contains("/");

	const bool bIsValidRootDir = FPaths::DirectoryExists(RootDirTextBox->GetText().ToString());

	const FString WorkspaceFolderName = FPaths::ConvertRelativePathToFull(WorkspaceNameTextBox->GetText().ToString());
	const bool bIsNameValid = !WorkspaceFolderName.IsEmpty();

	return bStreamIsValid && bIsValidRootDir && bIsNameValid;
}

#undef LOCTEXT_NAMESPACE
