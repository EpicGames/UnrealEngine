// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SWorkspaceWindow.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SEditableTextBox.h"

class UGSTab;

class SNewWorkspaceWindow final : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SNewWorkspaceWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SWorkspaceWindow> InParent, UGSTab* InTab);

private:
	FReply OnBrowseStreamClicked();
	FReply OnBrowseRootDirectoryClicked();

	FReply OnCreateClicked();
	FReply OnCancelClicked();

	bool IsCreateButtonEnabled() const;

	TSharedPtr<SEditableTextBox> StreamTextBox = nullptr;
	TSharedPtr<SEditableTextBox> RootDirTextBox = nullptr;
	TSharedPtr<SEditableTextBox> WorkspaceNameTextBox = nullptr;
	FString RootDirPreviousFolder;

	TSharedPtr<SWorkspaceWindow> Parent = nullptr;

	UGSTab* Tab = nullptr;
};
