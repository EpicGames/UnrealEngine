// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPopupTextWindow.h"

#include "Styling/AppStyle.h"

#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "PopupTextWindow"

void SPopupTextWindow::Construct(const FArguments& InArgs)
{
	TSharedRef<SScrollBar> InvisibleHorizontalScrollbar = SNew(SScrollBar)
		.AlwaysShowScrollbar(false)
		.Orientation(Orient_Horizontal);

	SWindow::Construct(SWindow::FArguments()
	.Title(InArgs._TitleText)
	.SizingRule(ESizingRule::Autosized)
	.MaxWidth(800)
	.MaxHeight(600)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(20.0f, 10.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SMultiLineEditableTextBox)
			.Padding(10.0f)
			.AlwaysShowScrollbars(InArgs._ShowScrollBars)
			.HScrollBar(InvisibleHorizontalScrollbar)
			.IsReadOnly(true)
			.AutoWrapText(true)
			.BackgroundColor(FLinearColor::Transparent)
			.Justification(InArgs._BodyTextJustification)
			.Text(InArgs._BodyText)
		]
		+SVerticalBox::Slot()
		.Padding(0.0f, 0.0f, 0.0f, 15.0f)
		.AutoHeight()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Top)
		[
			SNew(SPrimaryButton)
			.Text(LOCTEXT("PopupTextWindowOkayButtonText", "Ok"))
			.OnClicked_Lambda([this]()
			{
				SharedThis(this)->RequestDestroyWindow();
				return FReply::Handled();
			})
		]
	]);
}

#undef LOCTEXT_NAMESPACE
