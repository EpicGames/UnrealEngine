// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Text/TextLayout.h"
#include "Widgets/SWindow.h"

class SPopupTextWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SPopupTextWindow)
		: _BodyTextJustification(ETextJustify::Center)
		, _ShowScrollBars(false)
		{}
		SLATE_ARGUMENT(FText, TitleText)
		SLATE_ARGUMENT(FText, BodyText)
		SLATE_ARGUMENT(ETextJustify::Type, BodyTextJustification)
		SLATE_ARGUMENT(bool, ShowScrollBars)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
