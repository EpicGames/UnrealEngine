// Copyright Epic Games, Inc. All Rights Reserved.

#include "SScheduledSyncWindow.h"

#include "UGSLog.h"
#include "UGSTab.h"
#include "UGSTabManager.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "SPrimaryButton.h"
#include "SSimpleButton.h"
#include "SPopupTextWindow.h"

#include "UserSettings.h"

#define LOCTEXT_NAMESPACE "SScheduledSyncWindow"

// Todo: remove checkbox and make the positive action button the thing that schedules the sync
// Todo: show UI if a scheduled sync already exists and provide the ability to cancel it

void SScheduledSyncWindow::Construct(const FArguments& InArgs, UGSTab* InTab)
{
	Tab = InTab;
	UserSettings = Tab->GetUserSettings();

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("WindowTitle", "Schedule Sync"))
	.SizingRule(ESizingRule::Autosized)
	[
		SNew(SVerticalBox)
		// Hint text
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(20.0f, 20.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ScheduleSync", "Select a time of day to sync all or some projects."))
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(20.0f, 0.0f)
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(10.0f, 0.0f))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EnableDisableScheudleSync", "Enable Schedule Sync:"))
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SCheckBox)
					.ForegroundColor(FSlateColor::UseForeground())
					.ToolTipText(LOCTEXT("EnableDisableScheudleSyncCheckbox", "Enable or disable Schedule Sync"))
					.IsChecked(this, &SScheduledSyncWindow::HandleGetScheduleSyncChecked)
					.OnCheckStateChanged(this, &SScheduledSyncWindow::HandleScheduleSyncChanged)
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(20.0f, 10.0f)
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(10.0f, 0.0f))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SyncTimeTextBoxLabel", "Scheduled Sync Time:"))
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SEditableTextBox)
					.ClearKeyboardFocusOnCommit(false)
					.OnTextCommitted(this, &SScheduledSyncWindow::HandleTextBoxTextCommited)
					.SelectAllTextOnCommit(true)
					.Text(this, &SScheduledSyncWindow::HandleTextBoxText)
					.Justification(ETextJustify::Center)
				]
			]
		]
		// TODO widget to enter a time of day
		// TODO check box to enable it for all projects
		// TODO check box to enable each project if the previous one is not enabled
		// Buttons
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(20.0f, 10.0f, 20.0f, 20.0f)
		.VAlign(VAlign_Bottom)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(10.0f, 0.0f))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("OkButtonText", "Ok"))
					.OnClicked(this, &SScheduledSyncWindow::OnOkClicked)
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("CancelButtonText", "Close")) // Todo: temporary fix, change the text back to "Cancel" when canceling actually cancels changes
					.OnClicked(this, &SScheduledSyncWindow::OnCancelClicked)
				]
			]
		]
	]);

	// Lets make the default input our current saved ScheduleTime
	// Move to an FDateTime from a FTimespan due to FTimespan having an implicit +/- symbol for ToString methods. As well as am/pm strings
	FDateTime SavedScheduledTime = FDateTime(1, 1, 1, UserSettings->ScheduleTime.GetHours(), UserSettings->ScheduleTime.GetMinutes());
	Input = SavedScheduledTime.ToString(TEXT("%h:%M%a"));
}

FText SScheduledSyncWindow::HandleTextBoxText() const
{
	return FText::FromString(Input);
}

void SScheduledSyncWindow::HandleTextBoxTextCommited(const FText& NewText, ETextCommit::Type CommitInfo)
{
	FString NewTime = NewText.ToString();

	// Valid format for a yyyy.mm.dd-hh.mm.ss
	// turn our am/pm into a 1/0 to use to see if we need to add 12 hours when creating the FDateTime::Parse format
	NewTime.ReplaceInline(TEXT("am"), TEXT(" 1"));
	NewTime.ReplaceInline(TEXT("pm"), TEXT(" 0"));

	NewTime.ReplaceInline(TEXT(":"), TEXT(" "));

	// split up on a single delimiter
	TArray<FString> Tokens;
	NewTime.ParseIntoArray(Tokens, TEXT(" "), true);

	// make sure it parsed it properly (within reason)
	// we expect an hour, minute and am/pm
	if (Tokens.Num() != 3)
	{
		UE_LOGF(LogSlateUGS, Warning, "Invalid Time Format %ls, should be in the form of <HH:MM[am/pm]>", *NewText.ToString());
		return;
	}

	int32 Hour   = FCString::Atoi(*Tokens[0]);
	int32 Minute = FCString::Atoi(*Tokens[1]);
	int32 bIsAm  = FCString::Atoi(*Tokens[2]);

	if (!bIsAm)
	{
		Hour += 12;
	}

	// 1.1.1 is just a valid date which we dont care about but is required for parsing into a FDateTime
	FDateTime InputTime;

	bInputValid = FDateTime::Parse(FString::Printf(TEXT("1.1.1-%i.%i.0"), Hour, Minute), InputTime);

	if (bInputValid)
	{
		FTimespan NewScheduleTime = FTimespan(Hour, Minute, 0);
		Input = InputTime.ToString(TEXT("%h:%M%a"));

		if (NewScheduleTime != UserSettings->ScheduleTime)
		{
			UserSettings->ScheduleTime = NewScheduleTime;
			UserSettings->Save();

			Tab->GetTabManager()->SetupScheduledSync();
		}
	}
	else
	{
		UE_LOGF(LogSlateUGS, Warning, "Invalid Time Format %ls, should be in the form of <HH:MM[am/pm]>", *NewText.ToString());
	}
}

void SScheduledSyncWindow::HandleScheduleSyncChanged(ECheckBoxState InCheck)
{
	bool bNewValue = InCheck == ECheckBoxState::Checked;
	if (UserSettings->bScheduleEnabled != bNewValue)
	{
		UserSettings->bScheduleEnabled = bNewValue;
		UserSettings->Save();

		if (UserSettings->bScheduleEnabled)
		{
			Tab->GetTabManager()->SetupScheduledSync();
		}
		else
		{
			Tab->GetTabManager()->StopScheduledSyncTimer();
		}
	}
}

ECheckBoxState SScheduledSyncWindow::HandleGetScheduleSyncChecked() const
{
	return UserSettings->bScheduleEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FReply SScheduledSyncWindow::OnOkClicked()
{
	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();

	return FReply::Handled();
}

FReply SScheduledSyncWindow::OnCancelClicked()
{
	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
