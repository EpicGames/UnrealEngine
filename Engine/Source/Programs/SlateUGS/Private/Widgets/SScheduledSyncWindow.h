// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Styling/SlateColor.h"

class UGSTab;

namespace UGSCore
{
	struct FUserSettings;
}

class SScheduledSyncWindow final : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SScheduledSyncWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UGSTab* InTab);

private:
	/** Handles getting the text color of the editable text box. */
	FSlateColor HandleTextBoxForegroundColor() const;

	/** Handles getting the text to be displayed in the editable text box. */ 
	FText HandleTextBoxText() const;

	/** Handles committing the text in the editable text box. */
	void HandleTextBoxTextCommited(const FText& NewText, ETextCommit::Type CommitInfo);

	/** Handle the check box enable or disabling Schedule Sync */
	void HandleScheduleSyncChanged(ECheckBoxState InCheck);
	ECheckBoxState HandleGetScheduleSyncChecked() const;

	FReply OnOkClicked();
	FReply OnCancelClicked();

	bool bInputValid = false;
	FString Input;

	UGSTab* Tab = nullptr;
	TSharedPtr<UGSCore::FUserSettings> UserSettings;
};
