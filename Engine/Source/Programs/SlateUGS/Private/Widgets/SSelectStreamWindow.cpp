// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSelectStreamWindow.h"

#include "SlateUGSStyle.h"
#include "Framework/Application/SlateApplication.h"

#include "UGSTab.h"
#include "SGameSyncTab.h"
#include "SPrimaryButton.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "UGSNewWorkspaceWindow"

struct FStreamNode
{
	FStreamNode(const FString& InLabel, const FString& InFullStreamPath, bool bInIsStream)
		: Label(FText::FromString(InLabel))
		, FullStreamPath(InFullStreamPath)
		, bIsStream(bInIsStream) {}

	virtual ~FStreamNode() {}

	FText Label;
	FString FullStreamPath;
	bool bIsStream;
	TArray<TSharedPtr<FStreamNode>> Children;
};

void SSelectStreamWindow::Construct(const FArguments& InArgs, UGSTab* InTab, FString* OutSelectedStreamPath)
{
	Tab = InTab;
	SelectedStreamPath = OutSelectedStreamPath;

	PopulateStreamsTree();

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("WindowTitleSelect", "Select Stream"))
	.SizingRule(ESizingRule::FixedSize)
	.ClientSize(FVector2D(600, 500))
	[
		SNew(SBox)
		.Padding(30.0f, 15.0f, 30.0f, 0.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SEditableTextBox)
				// Todo: change hint and enable when filters are supported
				.HintText(LOCTEXT("FilterHint", "Filter (under construction, does not work yet)"))
				.IsEnabled(false)
			]
			+SVerticalBox::Slot()
			.Padding(0.0f, 15.0f, 0.0f, 0.0f)
			[
				SNew(STreeView<TSharedRef<FStreamNode>>)
				.TreeItemsSource(&StreamsTree)
				.OnGenerateRow(this, &SSelectStreamWindow::OnGenerateRow)
				.OnGetChildren(this, &SSelectStreamWindow::OnGetChildren)
				.OnSelectionChanged(this, &SSelectStreamWindow::OnSelectionChanged)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				.Padding(10.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 10.0f, 0.0f)
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("OkButtonText", "Ok"))
						.OnClicked(this, &SSelectStreamWindow::OnOkClicked)
						.IsEnabled(this, &SSelectStreamWindow::IsOkButtonEnabled)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Text(LOCTEXT("CancelButtonText", "Cancel"))
						.OnClicked(this, &SSelectStreamWindow::OnCancelClicked)
					]
				]
			]
		]
	]);
}

TSharedRef<ITableRow> SSelectStreamWindow::OnGenerateRow(TSharedRef<FStreamNode> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(STableRow<TSharedRef<FStreamNode>>, InOwnerTable)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(0.0f, 0.0f, 5.0f, 0.0f)
		// [
		// 	SNew(SImage)
		// 	// Todo: replace text with keys to actuall perforce depot/stream icons
		// 	.Image(FSlateUGSStyle::Get().GetBrush(InItem->bIsStream ? "FILL ME IN" : "FILL ME IN TOO))
		// ]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(InItem->Label)
		]
	];
}

void SSelectStreamWindow::OnGetChildren(TSharedRef<FStreamNode> InItem, TArray<TSharedRef<FStreamNode>>& OutChildren)
{
	for (int32 ChildIndex = 0; ChildIndex < InItem->Children.Num(); ++ChildIndex)
	{
		TSharedPtr<FStreamNode> Child = InItem->Children[ChildIndex];
		if (Child.IsValid())
		{
			OutChildren.Add(Child.ToSharedRef());
		}
	}
}

void SSelectStreamWindow::PopulateStreamsTree()
{
	TArray<FString> Streams = Tab->GetAllStreamNames();
	FString CurrentRoot = "";

	for (const FString& Stream : Streams)
	{
		TArray<FString> StreamPath;
		Stream.RightChop(2).ParseIntoArray(StreamPath, TEXT("/"));

		if (StreamPath[0] != CurrentRoot)
		{
			CurrentRoot = StreamPath[0];
			StreamsTree.Add(MakeShareable(new FStreamNode(StreamPath[0], Stream, false)));
		}

		FString CurrentChild = Stream.RightChop(CurrentRoot.Len() + 3);
		StreamsTree.Last()->Children.Add(MakeShareable(new FStreamNode(CurrentChild, Stream, true)));
	}
}

void SSelectStreamWindow::OnSelectionChanged(TSharedPtr<FStreamNode> NewSelection, ESelectInfo::Type SelectInfo)
{
	SelectedStream = NewSelection;
}

FReply SSelectStreamWindow::OnOkClicked()
{
	*SelectedStreamPath = SelectedStream->FullStreamPath;

	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();
	return FReply::Handled();
}

FReply SSelectStreamWindow::OnCancelClicked()
{
	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();
	return FReply::Handled();
}

bool SSelectStreamWindow::IsOkButtonEnabled() const
{
	return SelectedStream.IsValid() && SelectedStream->bIsStream;
}

#undef LOCTEXT_NAMESPACE
