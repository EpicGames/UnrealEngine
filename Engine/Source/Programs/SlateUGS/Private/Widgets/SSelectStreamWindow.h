// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Widgets/Input/SEditableTextBox.h"

class UGSTab;
struct FStreamNode;

class SSelectStreamWindow final : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SSelectStreamWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UGSTab* InTab, FString* OutSelectedStreamPath);

private:
	TSharedPtr<SEditableTextBox> FilterText;
	TSharedPtr<FStreamNode> SelectedStream;
	FString* SelectedStreamPath = nullptr;

	TArray<TSharedRef<FStreamNode>> StreamsTree;
	void PopulateStreamsTree();
	TSharedRef<ITableRow> OnGenerateRow(TSharedRef<FStreamNode> InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	void OnGetChildren(TSharedRef<FStreamNode> InItem, TArray<TSharedRef<FStreamNode>>& OutChildren);
	void OnSelectionChanged(TSharedPtr<FStreamNode> NewSelection, ESelectInfo::Type SelectInfo);

	FReply OnOkClicked();
	FReply OnCancelClicked();

	bool IsOkButtonEnabled() const;

	UGSTab* Tab = nullptr;
};
