// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSettingsWindow.h"
#include "UGSTab.h"

#include "UserSettings.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "SSettingsWindow"

// Todo: Make settings only save when the positive action button is hit

void SSettingsWindow::Construct(const FArguments& InArgs, UGSTab* InTab)
{
	Tab = InTab;
	UserSettings = Tab->GetUserSettings();

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("WindowTitle", "Settings"))
	.SizingRule(ESizingRule::FixedSize)
	.ClientSize(FVector2D(1100, 800))
	[
		SNew(SVerticalBox)
		// Hint text
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(20.0f, 20.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SettingsHintText", "Settings for Unreal Game Sync"))
		]
		// Start of Settings
		+SVerticalBox::Slot()
		.Padding(20.0f, 0.0f)
		[
			SNew(SVerticalBox)
			// After Sync Settings
			+SVerticalBox::Slot()
			.FillHeight(0.15f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				.Padding(0.0f, 10.0f)
				[
					SNew(SHeader)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AfterSync", "After Sync Settings"))
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Center)
				.Padding(0.0f, 10.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SCheckBox)
						.ForegroundColor(FSlateColor::UseForeground())
						.IsChecked(this, &SSettingsWindow::HandleGetBuildChecked)
						.OnCheckStateChanged(this, &SSettingsWindow::HandleBuildChanged)
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.Padding(FMargin(4.0, 2.0))
							[
								SNew(STextBlock)
								.Text(LOCTEXT("EnableBuildAfterSync", "Build"))
							]
						]
					]
					+SHorizontalBox::Slot()
					.Padding(200.0f, 0.0f)
					[
						SNew(SCheckBox)
						.ForegroundColor(FSlateColor::UseForeground())
						.IsChecked(this, &SSettingsWindow::HandleGetRunChecked)
						.OnCheckStateChanged(this, &SSettingsWindow::HandleRunChanged)
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.Padding(FMargin(4.0, 2.0))
							[
								SNew(STextBlock)
								.Text(LOCTEXT("EnableRunAfterSync", "Run"))
							]
						]
					]
					+SHorizontalBox::Slot()
					[
						SNew(SCheckBox)
						.ForegroundColor(FSlateColor::UseForeground())
						.IsChecked(this, &SSettingsWindow::HandleGetOpenSolutionChecked)
						.OnCheckStateChanged(this, &SSettingsWindow::HandleOpenSolutionChanged)
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.Padding(FMargin(4.0, 2.0))
							[
								SNew(STextBlock)
								.Text(LOCTEXT("EnableOpenSolutionAfterSync", "Open Solution"))
							]
						]
					]
				]
				// Other
				+SVerticalBox::Slot()
				.FillHeight(0.6f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					.AutoHeight()
					.Padding(0.0f, 10.0f)
					[
						SNew(SHeader)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Other Settings", "Other Settings"))
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Center)
					.Padding(0.0f, 10.0f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						[
							SNew(SCheckBox)
							.ForegroundColor(FSlateColor::UseForeground())
							.IsChecked(this, &SSettingsWindow::HandleGetSyncCompiledEditor)
							.OnCheckStateChanged(this, &SSettingsWindow::HandleSyncCompiledEditor)
							[
								SNew(SBox)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								.Padding(FMargin(4.0, 2.0))
								[
									SNew(STextBlock)
									.Text(LOCTEXT("EnableSyncPrecompiledEditor", "Sync Precompiled Editor"))
								]
							]
						]
					]
				]
			]
		]
		// Buttons
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 20.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 10.0f, 0.0f)
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(10.0f, 0.0f))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("OkButtonText", "Ok"))
					.OnClicked(this, &SSettingsWindow::OnOkClicked)
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("CancelButtonText", "Close")) // Todo: temporary fix, change the text back to "Cancel" when canceling actually cancels changes
					.OnClicked(this, &SSettingsWindow::OnCancelClicked)
				]
			]
		]
	]);
}

void SSettingsWindow::HandleBuildChanged(ECheckBoxState InCheck)
{
	bool bNewValue = InCheck == ECheckBoxState::Checked;

	if (UserSettings->bBuildAfterSync != bNewValue)
	{
		UserSettings->bBuildAfterSync = bNewValue;
		UserSettings->Save();
	}
}

ECheckBoxState SSettingsWindow::HandleGetBuildChecked() const
{
	return UserSettings->bBuildAfterSync ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SSettingsWindow::HandleRunChanged(ECheckBoxState InCheck)
{
	bool bNewValue = InCheck == ECheckBoxState::Checked;

	if (UserSettings->bRunAfterSync != bNewValue)
	{
		UserSettings->bRunAfterSync = bNewValue;
		UserSettings->Save();
	}
}

ECheckBoxState SSettingsWindow::HandleGetRunChecked() const
{
	return UserSettings->bRunAfterSync ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SSettingsWindow::HandleOpenSolutionChanged(ECheckBoxState InCheck)
{
	bool bNewValue = InCheck == ECheckBoxState::Checked;

	if (UserSettings->bOpenSolutionAfterSync != bNewValue)
	{
		UserSettings->bOpenSolutionAfterSync = bNewValue;
		UserSettings->Save();
	}
}

ECheckBoxState SSettingsWindow::HandleGetOpenSolutionChecked() const
{
	return UserSettings->bOpenSolutionAfterSync ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SSettingsWindow::HandleSyncCompiledEditor(ECheckBoxState InCheck)
{
	bool bNewValue = InCheck == ECheckBoxState::Checked;

	if (UserSettings->bSyncPrecompiledEditor != bNewValue)
	{
		UserSettings->bSyncPrecompiledEditor = bNewValue;
		UserSettings->Save();

		// If we update this setting, update the build list to refresh filtering
		Tab->UpdateGameTabBuildList();
	}
}

ECheckBoxState SSettingsWindow::HandleGetSyncCompiledEditor() const
{
	return UserSettings->bSyncPrecompiledEditor ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FReply SSettingsWindow::OnOkClicked()
{
	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();

	return FReply::Handled();
}

FReply SSettingsWindow::OnCancelClicked()
{
	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
