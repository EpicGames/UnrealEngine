// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"

#include "SCheckBoxList.h"

class UGSTab;

namespace UGSCore
{
	struct FUserSettings;
}

class SSettingsWindow final: public SWindow
{
public:
	SLATE_BEGIN_ARGS(SSettingsWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UGSTab* InTab);

private:
	/** Handle the check box enable or disabling for AfterSync settings */
	void HandleBuildChanged(ECheckBoxState InCheck);
	ECheckBoxState HandleGetBuildChecked() const;

	void HandleRunChanged(ECheckBoxState InCheck);
	ECheckBoxState HandleGetRunChecked() const;

	void HandleOpenSolutionChanged(ECheckBoxState InCheck);
	ECheckBoxState HandleGetOpenSolutionChecked() const;

	void HandleSyncCompiledEditor(ECheckBoxState InCheck);
	ECheckBoxState HandleGetSyncCompiledEditor() const;

	FReply OnOkClicked();
	FReply OnCancelClicked();

	UGSTab* Tab = nullptr;
	TSharedPtr<UGSCore::FUserSettings> UserSettings;
};
