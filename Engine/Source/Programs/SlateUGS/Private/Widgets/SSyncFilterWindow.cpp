// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSyncFilterWindow.h"

#include "UGSTab.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SHeader.h"
#include "SPrimaryButton.h"
#include "SPopupTextWindow.h"

#define LOCTEXT_NAMESPACE "SSyncFilterWindow"

void SSyncFilterWindow::ConstructSyncFilters()
{
	WorkspaceCategoriesCurrent = Tab->GetSyncCategories(SyncCategoryType::CurrentWorkspace);

	SyncFiltersCurrent = SNew(SCheckBoxList)
		.ItemHeaderLabel(LOCTEXT("CheckBoxListCurrent", "Sync Filters for the current workspace"))
		.IncludeGlobalCheckBoxInHeaderRow(false);

	for (const UGSCore::FWorkspaceSyncCategory& Category : WorkspaceCategoriesCurrent)
	{
		TSharedRef<SBox> CheckBoxWrapper = SNew(SBox)
			.Padding(5.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Category.Name))
			];

		SyncFiltersCurrent->AddItem(CheckBoxWrapper, Category.bEnable);
	}

	WorkspaceCategoriesAll = Tab->GetSyncCategories(SyncCategoryType::AllWorkspaces);

	SyncFiltersAll = SNew(SCheckBoxList)
		.ItemHeaderLabel(LOCTEXT("CheckBoxListAll", "Sync Filters for all workspaces"))
		.IncludeGlobalCheckBoxInHeaderRow(false);

	for (const UGSCore::FWorkspaceSyncCategory& Category : WorkspaceCategoriesAll)
	{
		TSharedRef<SBox> CheckBoxWrapper = SNew(SBox)
			.Padding(5.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Category.Name))
			];

		SyncFiltersAll->AddItem(CheckBoxWrapper, Category.bEnable);
	}
}

void SSyncFilterWindow::ConstructCustomSyncViewTextBoxes()
{
	TSharedRef<SScrollBar> InvisibleHorizontalScrollbar = SNew(SScrollBar) // Todo: this code is duplicated in SPopupTextWindow.cpp
		.AlwaysShowScrollbar(false)										   // Is there a simpler way to make the horizontal scroll bar invisible?
		.Orientation(Orient_Horizontal);								   // If not, I guess we should factor this out into a tiny widget?

	SAssignNew(CustomSyncViewCurrent, SMultiLineEditableTextBox)
	.Padding(10.0f)
	.AutoWrapText(true)
	.AlwaysShowScrollbars(true)
	.HScrollBar(InvisibleHorizontalScrollbar)
	.BackgroundColor(FLinearColor::Transparent)
	.Justification(ETextJustify::Left)
	.Text(FText::FromString(FString::Join(Tab->GetSyncViews(SyncCategoryType::CurrentWorkspace), TEXT("\n"))));

	SAssignNew(CustomSyncViewAll, SMultiLineEditableTextBox)
	.Padding(10.0f)
	.AutoWrapText(true)
	.AlwaysShowScrollbars(true)
	.HScrollBar(InvisibleHorizontalScrollbar)
	.BackgroundColor(FLinearColor::Transparent)
	.Justification(ETextJustify::Left)
	.Text(FText::FromString(FString::Join(Tab->GetSyncViews(SyncCategoryType::AllWorkspaces), TEXT("\n"))));
}

void SSyncFilterWindow::Construct(const FArguments& InArgs, UGSTab* InTab)
{
	Tab = InTab;

	ConstructSyncFilters();
	ConstructCustomSyncViewTextBoxes();

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("WindowTitle", "Sync Filters"))
	.SizingRule(ESizingRule::FixedSize)
	.ClientSize(FVector2D(1100, 800))
	[
		SNew(SVerticalBox)
		// Hint text
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(20.0f, 20.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SyncFilterHintText", "Files synced from Perforce may be filtered by a custom stream view, and list of predefined categories. Settings for the current workspace override defaults for all workspaces."))
		]
		// Sync filters
		+SVerticalBox::Slot()
		.Padding(20.0f, 0.0f)
		[
			SNew(SVerticalBox)
			// General
			+SVerticalBox::Slot()
			.FillHeight(0.15f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoHeight()
				.Padding(0.0f, 10.0f)
				[
					SNew(SHeader)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("General", "General"))
					]
				]
			]
			// Categories
			+SVerticalBox::Slot()
			.FillHeight(0.6f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoHeight()
				.Padding(0.0f, 10.0f)
				[
					SNew(SHeader)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Categories", "Categories"))
					]
				]
				+SVerticalBox::Slot()
				.VAlign(VAlign_Fill)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SyncFiltersCurrent.ToSharedRef()
					]
					+SHorizontalBox::Slot()
					[
						SyncFiltersAll.ToSharedRef()
					]
				]
			]
			// Custom View
			+SVerticalBox::Slot()
			.FillHeight(0.35f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoHeight()
				.Padding(0.0f, 10.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SHeader)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("CustomView", "Custom View"))
						]
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(20.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SButton)
						.Text(LOCTEXT("CustomViewSyntax", "Syntax"))
						.OnClicked(this, &SSyncFilterWindow::OnCustomViewSyntaxClicked)
					]
				]
				+SVerticalBox::Slot()
				.VAlign(VAlign_Fill)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 10.0f, 0.0f)
					[
						CustomSyncViewCurrent.ToSharedRef()
					]
					+SHorizontalBox::Slot()
					.Padding(10.0f, 0.0f, 0.0f, 0.0f)
					[
						CustomSyncViewAll.ToSharedRef()
					]
				]
			]
		]
		// Buttons
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 20.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(20.0f, 0.0f, 0.0f, 0.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("ShowCombinedFilterButtonText", "Show Combined Filter"))
				.OnClicked(this, &SSyncFilterWindow::OnShowCombinedFilterClicked)
			]
			+SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 10.0f, 0.0f)
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(10.0f, 0.0f))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("SaveButtonText", "Save"))
					.OnClicked(this, &SSyncFilterWindow::OnSaveClicked)
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("CancelButtonText", "Cancel"))
					.OnClicked(this, &SSyncFilterWindow::OnCancelClicked)
				]
			]
		]
	]);
}

FReply SSyncFilterWindow::OnShowCombinedFilterClicked()
{
	TSharedRef<SPopupTextWindow> CombinedFilterWindow = SNew(SPopupTextWindow)
		.TitleText(LOCTEXT("CombinedSyncFilterWindowTitle", "Combined Sync Filter"))
		.BodyText(FText::FromString(FString::Join(Tab->GetCombinedSyncFilter(), TEXT("\n"))))
		.BodyTextJustification(ETextJustify::Left)
		.ShowScrollBars(true);
	FSlateApplication::Get().AddModalWindow(CombinedFilterWindow, SharedThis(this), false);
	return FReply::Handled();
}

FReply SSyncFilterWindow::OnCustomViewSyntaxClicked()
{
	FText Body = FText::FromString(
		"Specify a custom view of the stream using Perforce-style wildcards, one per line.\n\n"
		"  - All files are visible by default.\n"
		"  - To exclude files matching a pattern, prefix it with a '-' character (eg. -/Engine/Documentation/...)\n"
		"  - Patterns may match any file fragment (eg. *.pdb), or may be rooted to the branch (eg. /Engine/Binaries/.../*.pdb).\n\n"
		"The view for the current workspace will be appended to the view shared by all workspaces."
	);
	TSharedRef<SPopupTextWindow> CombinedFilterWindow = SNew(SPopupTextWindow)
		.TitleText(LOCTEXT("CustomSyncFilterSyntaxWindow", "Custom Sync Filter Syntax"))
		.BodyText(Body)
		.BodyTextJustification(ETextJustify::Left);
	FSlateApplication::Get().AddModalWindow(CombinedFilterWindow, SharedThis(this), false);
	return FReply::Handled();
}

FReply SSyncFilterWindow::OnSaveClicked()
{
	// Only store the delta from defaults, (all built-in categories default to enabled):
	//   checked   → matches default → not stored
	//   unchecked → differs from default → stored in ExcludeCategories
	TArray<FGuid> SyncExcludedCategoriesCurrent;
	TArray<bool> EnabledValuesCurrent = SyncFiltersCurrent->GetValues();
	for (int CategoryIndex = 0; CategoryIndex < WorkspaceCategoriesCurrent.Num(); CategoryIndex++)
	{
		if (!EnabledValuesCurrent[CategoryIndex])
		{
			SyncExcludedCategoriesCurrent.Add(WorkspaceCategoriesCurrent[CategoryIndex].UniqueId);
		}
	}

	TArray<FGuid> SyncExcludedCategoriesAll;
	TArray<bool> EnabledValuesAll = SyncFiltersAll->GetValues();
	for (int CategoryIndex = 0; CategoryIndex < WorkspaceCategoriesAll.Num(); CategoryIndex++)
	{
		if (!EnabledValuesAll[CategoryIndex])
		{
			SyncExcludedCategoriesAll.Add(WorkspaceCategoriesAll[CategoryIndex].UniqueId);
		}
	}

	TArray<FString> SyncViewCurrent;
	CustomSyncViewCurrent->GetPlainText().ToString().ParseIntoArray(SyncViewCurrent, TEXT("\n"));

	TArray<FString> SyncViewAll;
	CustomSyncViewAll->GetPlainText().ToString().ParseIntoArray(SyncViewAll, TEXT("\n"));

	TArray<FGuid> EmptyIncludeCategories;
	Tab->OnSyncFilterWindowSaved(SyncViewCurrent, EmptyIncludeCategories, SyncExcludedCategoriesCurrent, SyncViewAll, EmptyIncludeCategories, SyncExcludedCategoriesAll);

	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();
	return FReply::Handled();
}

FReply SSyncFilterWindow::OnCancelClicked()
{
	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
