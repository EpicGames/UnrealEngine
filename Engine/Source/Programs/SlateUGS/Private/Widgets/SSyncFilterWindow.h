// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

#include "Workspace.h"

#include "SCheckBoxList.h"

class UGSTab;

class SSyncFilterWindow final : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SSyncFilterWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UGSTab* InTab);

private:
	FReply OnShowCombinedFilterClicked();
	FReply OnCustomViewSyntaxClicked();
	FReply OnSaveClicked();
	FReply OnCancelClicked();

	void ConstructSyncFilters();
	void ConstructCustomSyncViewTextBoxes();

	UGSTab* Tab = nullptr;
	TArray<UGSCore::FWorkspaceSyncCategory> WorkspaceCategoriesCurrent;
	TArray<UGSCore::FWorkspaceSyncCategory> WorkspaceCategoriesAll;

	TSharedPtr<SCheckBoxList> SyncFiltersCurrent;
	TSharedPtr<SCheckBoxList> SyncFiltersAll;
	TSharedPtr<SMultiLineEditableTextBox> CustomSyncViewCurrent;
	TSharedPtr<SMultiLineEditableTextBox> CustomSyncViewAll;

};
