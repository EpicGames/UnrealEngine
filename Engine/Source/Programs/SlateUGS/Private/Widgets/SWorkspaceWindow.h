// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Widgets/Input/SEditableTextBox.h"

class UGSTab;

class SWorkspaceWindow final : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SWorkspaceWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UGSTab* InTab);

	void SetWorkspaceTextBox(FText Text) const;

private:
	FReply OnOkClicked();
	FReply OnCancelClicked();

	FReply OnBrowseClicked();
	FString PreviousProjectPath;

	FReply OnNewClicked();

	bool bIsLocalFileSelected = true;
	TSharedPtr<SEditableTextBox> LocalFileTextBox = nullptr;
	FString WorkspacePathText;

	TSharedPtr<SEditableTextBox> WorkspaceNameTextBox = nullptr;

	UGSTab* Tab = nullptr;
};
