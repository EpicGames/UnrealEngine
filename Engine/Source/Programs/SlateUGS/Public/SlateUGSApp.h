// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateUGS.h"

/**
 * Run the SlateUGS .
 */
int RunSlateUGS(const TCHAR* Commandline);
