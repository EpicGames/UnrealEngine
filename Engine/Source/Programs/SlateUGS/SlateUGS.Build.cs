// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateUGS : ModuleRules
{
	public SlateUGS(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"UGSCore",
				"AppFramework",
				"Core",
				"ApplicationCore",
				"Projects",
				"Slate",
				"SlateCore",
				"StandaloneRenderer",
				"ToolWidgets",
				"PakFile",
				"DesktopPlatform", // Open file dialog
			}
		);

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnixCommonStartup"
				}
			);
		}
	}
}
