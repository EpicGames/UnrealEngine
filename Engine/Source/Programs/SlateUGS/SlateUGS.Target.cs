// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class SlateUGSTarget : TargetRules
{
	public SlateUGSTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		LaunchModuleName = "SlateUGS";

		bBuildDeveloperTools = false;

		// SlateUGS doesn't ever compile with the engine linked in
		bCompileAgainstEngine = false;

		// We need CoreUObject compiled in as the source code access module requires it
		bCompileAgainstCoreUObject = true;

		// SlateUGS.exe has no exports, so no need to verify that a .lib and .exp file was emitted by
		// the linker.
		bHasExports = false;

		// Make sure to get all code in SlateEditorStyle compiled in
        bBuildDeveloperTools = true;
	}
}
